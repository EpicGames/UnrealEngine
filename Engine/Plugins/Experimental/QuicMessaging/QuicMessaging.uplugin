{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "QUIC Messaging",
	"Description": "Adds a QUIC based transport layer to the messaging sub-system for sending and receiving messages between networked computers and devices.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"SupportedPrograms": [
		"UnrealFrontend",
		"UnrealMultiUserServer",
		"UnrealMultiUserSlateServer",
		"UnrealRecoverySvc",
		"UnrealLightmass",
		"CrashReportClientEditor",
		"CoopMultiUserServer",
		"LiveLinkHub"
	],
	"Modules": [
		{
			"Name": "QuicMessaging",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"ProgramAllowList": [
				"UnrealFrontend",
				"UnrealMultiUserServer",
				"UnrealMultiUserSlateServer",
				"UnrealRecoverySvc",
				"UnrealLightmass",
				"CrashReportClientEditor",
				"CoopMultiUserServer",
				"LiveLinkHub"
			]
		},
		{
			"Name": "QuicMessagingTransport",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux" ],
			"ProgramAllowList": [
				"UnrealFrontend",
				"UnrealMultiUserServer",
				"UnrealMultiUserSlateServer",
				"UnrealRecoverySvc",
				"UnrealLightmass",
				"CrashReportClientEditor",
				"CoopMultiUserServer",
				"LiveLinkHub"
			]
		}
	],
	"Plugins": [
		{
			"Name": "MsQuic",
			"Enabled": true
		}
	]
}
