// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicMessagingPrivate.h"
#include "QuicMessageProcessor.h"
#include "QuicEndpointConfig.h"

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "MessageBridgeBuilder.h"
#include "Misc/CoreMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Containers/Ticker.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Stats/Stats.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
#endif

#include "MsQuicRuntimeModule.h"
#include "Features/IModularFeatures.h"
#include "IQuicNetworkMessagingExtension.h"
#include "MessageEndpoint.h"
#include "QuicTransportMessages.h"
#include "Interfaces/IPluginManager.h"
#include "Shared/QuicMessagingSettings.h"
#include "Transport/QuicMessageTransport.h"

DEFINE_LOG_CATEGORY(LogQuicMessaging);

#define LOCTEXT_NAMESPACE "FQuicMessagingModule"


/**
 * Implements the QuicMessagingModule and the network messaging extension modular feature.
 */
class FQuicMessagingModule
    : public FSelfRegisteringExec
    , public IQuicNetworkMessagingExtension
    , public IModuleInterface
{
public:

    //~ FSelfRegisteringExec interface

    virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
    {
        if (!FParse::Command(&Cmd, TEXT("QUICMESSAGING")))
		{
			return false;
		}

        if (FParse::Command(&Cmd, TEXT("STATUS")))
		{
			UQuicMessagingSettings* Settings = GetMutableDefault<UQuicMessagingSettings>();

			// Bridge status
			if (MessageBridge.IsValid())
			{
				if (MessageBridge->IsEnabled())
				{
					Ar.Log(TEXT("Message Bridge: Initialized and enabled"));
				}
				else
				{
					Ar.Log(TEXT("Message Bridge: Initialized, but disabled"));
				}
			}
			else
			{
				Ar.Log(TEXT("Message Bridge: Not initialized."));
			}

			// Bridge settings
			if (Settings->UnicastEndpoint.IsEmpty())
			{
				Ar.Logf(TEXT("    Unicast Endpoint: %s (default)"),
					*FIPv4Endpoint::Any.ToString());
			}
			else
			{
				Ar.Logf(TEXT("    Unicast Endpoint: %s"), *Settings->UnicastEndpoint);
			}

			if (Settings->StaticEndpoints.Num() > 0)
			{
				Ar.Log(TEXT("    Static Endpoints:"));

				for (const auto& StaticEndpoint : Settings->StaticEndpoints)
				{
					Ar.Logf(TEXT("        %s"), *StaticEndpoint);
				}
			}
			else
			{
				Ar.Log(TEXT("    Static Endpoints: None"));
			}
        }
        else if (FParse::Command(&Cmd, TEXT("RESTART")))
        {
            RestartServices();
        }
        else if (FParse::Command(&Cmd, TEXT("SHUTDOWN")))
        {
            ShutdownBridge();
        }
        else
        {
			// Show usage
			Ar.Log(TEXT("Usage: QUICMESSAGING <Command>"));
			Ar.Log(TEXT(""));
			Ar.Log(TEXT("Command"));
			Ar.Log(TEXT("    RESTART = Restarts the message bridge, if enabled"));
			Ar.Log(TEXT("    SHUTDOWN = Shut down the message bridge, if running"));
			Ar.Log(TEXT("    STATUS = Displays the status of the QUIC message transport"));
        }

        return true;
    }

public:

    // IModuleInterface
    virtual void StartupModule() override
    {
		if (!IsSupportEnabled())
		{
			return;
		}

        // Load networking dependency
		if (!FModuleManager::Get().LoadModule(TEXT("Networking")))
        {
            UE_LOGF(LogQuicMessaging, Error,
				"The required module 'Networking' failed to load."
				"Plug-in 'QUIC Messaging' cannot be used.");

			return;
        }

		// Initiate MsQuicRuntime (loads the DLLs)
		if (!FMsQuicRuntimeModule::InitRuntime())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingModule] Could not initialize MsQuicRuntime.");

			return;
		}

		// Hook to the PreExit callback, needed to execute UObject related shutdowns
		FCoreDelegates::OnPreExit.AddRaw(
			this, &FQuicMessagingModule::HandleAppPreExit);

#if WITH_EDITOR
    	
        // Register settings
        if (ISettingsModule* SettingsModule
			= FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
        {
            const ISettingsSectionPtr SettingsSection
        		= SettingsModule->RegisterSettings("Project", "Plugins", "QuicMessaging",
                LOCTEXT("QuicMessagingSettingsName", "QUIC Messaging"),
                LOCTEXT("QuicMessagingSettingsDescription", "Configure the QUIC Messaging plugin."),
                GetMutableDefault<UQuicMessagingSettings>()
            );

            if (SettingsSection.IsValid())
            {
                SettingsSection->OnModified().BindRaw(
					this, &FQuicMessagingModule::HandleSettingsSaved);
            }
        }
#endif

        // Parse additional command line args
        ParseCommandLine(GetMutableDefault<UQuicMessagingSettings>(), FCommandLine::Get());

        // Register application events
        const UQuicMessagingSettings& Settings = *GetDefault<UQuicMessagingSettings>();

        if (Settings.bStopServiceWhenAppDeactivates)
        {
            FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(
				this, &FQuicMessagingModule::HandleApplicationHasReactivated);

			FCoreDelegates::ApplicationWillDeactivateDelegate.AddRaw(
				this, &FQuicMessagingModule::HandleApplicationWillDeactivate);
        }

		// Create the endpoints Guid
    	CreateEndpointGuid();

		// Restart services to apply changes
        RestartServices();

        IModularFeatures::Get().RegisterModularFeature(
			INetworkMessagingExtension::ModularFeatureName, this);
    }

    virtual void ShutdownModule() override
    {
        // Un-register application events
		FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
		FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);

		// Unhook AppPreExit and call it
		FCoreDelegates::OnPreExit.RemoveAll(this);
		HandleAppPreExit();

#if WITH_EDITOR
        // Un-register settings

		if (ISettingsModule* SettingsModule
			= FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
		{
			SettingsModule->UnregisterSettings(
				"Project", "Plugins", "QuicMessaging");
		}
#endif

        // Shut down services
        ShutdownBridge();

        IModularFeatures::Get().UnregisterModularFeature(
			INetworkMessagingExtension::ModularFeatureName, this);
    }

    virtual bool SupportsDynamicReloading() override
    {
        return true;
    }

    // INetworkMessagingExtension interface
    virtual FName GetName() const override
    {
        return QuicMessagingName;
    }

    virtual bool IsSupportEnabled() const override
    {
#if !IS_PROGRAM && UE_BUILD_SHIPPING && !(defined(ALLOW_QUIC_MESSAGING_SHIPPING) && ALLOW_QUIC_MESSAGING_SHIPPING)
        return false;
#else

    	// Disallow unsupported platforms
		if (!FPlatformMisc::SupportsMessaging())
		{
			return false;
		}

		// Always allow in standalone Slate applications
		if (!FApp::IsGame() && !IsRunningCommandlet())
		{
			return true;
		}

		// Otherwise allow if explicitly desired
		if (FParse::Param(FCommandLine::Get(), TEXT("Messaging")))
		{
			return true;
		}

        // Check the project setting
        const UQuicMessagingSettings& Settings = *GetDefault<UQuicMessagingSettings>();
        return Settings.EnabledByDefault;
#endif
    }

    virtual void RestartServices() override
    {
        const UQuicMessagingSettings& Settings = *GetDefault<UQuicMessagingSettings>();

        if (Settings.EnableTransport)
        {
            InitializeBridge();
        }
        else
        {
            ShutdownBridge();
        }
    }

    virtual void ShutdownServices() override
    {
        ShutdownBridge();

        AdditionalStaticEndpoints.Empty();
    }

	virtual bool CanProvideNetworkStatistics() const override
	{
		return true;
	}

	virtual FMessageTransportStatistics GetLatestNetworkStatistics(FGuid NodeId) const override
    {
		if (const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin())
		{
			return Transport->GetLatestStatistics(NodeId);
		}

		return {};
    }

	virtual FGuid GetNodeIdFromAddress(const FMessageAddress& MessageAddress) const override
    {
		return MessageBridge->LookupAddress(MessageAddress);
    }

	virtual FOnOutboundTransferDataUpdated& OnOutboundTransferUpdatedFromThread() override
	{
		return UE::Private::MessageProcessor::OnOutboundUpdated();
	}

	virtual FOnInboundTransferDataUpdated& OnInboundTransferUpdatedFromThread() override
	{
		return UE::Private::MessageProcessor::OnInboundUpdated();
	}

	virtual TArray<FString> GetListeningAddresses() const override
    {
		if (const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin())
		{
			TArray<FString> StringifiedEndpoints;
			TArray<FIPv4Endpoint> Endpoints = Transport->GetListeningAddresses();

			for (const FIPv4Endpoint& Endpoint : Endpoints)
			{
				StringifiedEndpoints.Add(Endpoint.ToString());
			}

			return StringifiedEndpoints;
		}

    	return {};
    }

	virtual TArray<FString> GetKnownEndpoints() const override
    {
		if (const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin())
		{
			TArray<FString> StringifiedEndpoints;
			TArray<FIPv4Endpoint> Endpoints = Transport->GetKnownEndpoints();

			for (const FIPv4Endpoint& Endpoint : Endpoints)
			{
				StringifiedEndpoints.Add(Endpoint.ToString());
			}

			return StringifiedEndpoints;
		}

    	return {};
    }

    virtual void AddEndpoint(const FString& InEndpoint) override
    {
        if (const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin())
        {
			if (InEndpoint == "")
			{
				return;
			}

            FScopeLock StaticEndpointsLock(&StaticEndpointsCS);
            FIPv4Endpoint OutEndpoint;
        	
            if (ParseEndpoint(InEndpoint, OutEndpoint)
				&& !AdditionalStaticEndpoints.Contains(OutEndpoint))
            {	
                AdditionalStaticEndpoints.Add(OutEndpoint);
                Transport->AddStaticEndpoint(OutEndpoint);
            }
        }
    }

	virtual void RemoveEndpoint(const FString& InEndpoint) override
	{
		if (const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin())
		{
			FScopeLock StaticEndpointsLock(&StaticEndpointsCS);
			FIPv4Endpoint OutEndpoint;

			if (ParseEndpoint(InEndpoint, OutEndpoint)
				&& AdditionalStaticEndpoints.Contains(OutEndpoint))
			{
				AdditionalStaticEndpoints.Remove(OutEndpoint);
				Transport->RemoveStaticEndpoint(OutEndpoint);
			}
		}
	}

public:

	/** IQuicNetworkMessagingExtension implementation. */

	virtual FGuid GetEndpointGuid() override
    {
		const UQuicMessagingSettings& Settings = *GetDefault<UQuicMessagingSettings>();

    	return Settings.EndpointGuid;
    }

	virtual TOptional<FGuid> GetNodeId(const FString& RemoteEndpoint) override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::GetNodeId] Transport pointer not valid.");

			return TOptional<FGuid>();
		}

		FIPv4Endpoint ParsedRemoteEndpoint;

		if (!FIPv4Endpoint::Parse(RemoteEndpoint, ParsedRemoteEndpoint))
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::GetNodeId] Endpoint string could not be parsed.");

			return TOptional<FGuid>();
		}

		return Transport->GetNodeId(ParsedRemoteEndpoint);
	}

	virtual void SetMaxAuthenticationMessageSize(const uint32 MaxBytes) override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::SetMaxAuthenticationMessageSize] "
				"Transport pointer not valid.");

			return;
		}

		Transport->SetMaxAuthenticationMessageSize(MaxBytes);
	}

	virtual bool IsNodeAuthenticated(const FGuid& NodeId) const override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::IsNodeAuthenticated] Transport pointer not valid.");

			return false;
		}

		return Transport->IsNodeAuthenticated(NodeId);
	}

	virtual void SetNodeAuthenticated(const FGuid& NodeId) override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::SetNodeAuthenticated] Transport pointer not valid.");

			return;
		}

		Transport->SetNodeAuthenticated(NodeId);
	}

	virtual void DisconnectNode(const FGuid& NodeId) override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::DisconnectNode] Transport pointer not valid.");

			return;
		}

		Transport->DisconnectNode(NodeId);
	}

	virtual void SetConnectionCooldown(
		const bool bEnabled, const uint32 MaxAttempts, const uint32 PeriodSeconds,
		const uint32 CooldownSeconds, const uint32 CooldownMaxSeconds) override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::SetConnectionCooldown] Transport pointer not valid.");

			return;
		}

		Transport->SetConnectionCooldown(
			bEnabled, MaxAttempts, PeriodSeconds, CooldownSeconds, CooldownMaxSeconds);
	}

	virtual bool TransportAuthMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    	const FGuid& Recipient) override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::TransportAuthMessage] Transport pointer not valid.");

			return false;
		}

		return Transport->TransportAuthMessage(Context, Recipient);
	}

	virtual bool TransportAuthResponseMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    	const FGuid& Recipient) override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error,
				"[QuicMessagingAPI::TransportAuthResponseMessage]"
					"Transport pointer not valid.");

			return false;
		}

		return Transport->TransportAuthResponseMessage(Context, Recipient);
	}

	virtual FOnQuicMetaMessageReceived& OnQuicMetaMessageReceived() override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Fatal,
				"[QuicMessagingAPI::OnQuicMetaMessageReceived] Transport pointer not valid.");
		}

		return Transport->OnMetaMessageReceived();
	}

	virtual FOnQuicClientConnectionChanged& OnQuicClientConnectionChanged() override
	{
		const TSharedPtr<FQuicMessageTransport> Transport = WeakBridgeTransport.Pin();

		if (!Transport.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Fatal,
				"[QuicMessagingAPI::OnQuicClientConnectionChanged] Transport pointer not valid.");
		}

		return Transport->OnClientConnectionChanged();
	}

protected:

	static bool ParseEndpoint(const FString& InEndpointString,
		FIPv4Endpoint& OutEndpoint, bool bSupportHostname = true)
	{
		bool bParsedEndpoint = FIPv4Endpoint::Parse(InEndpointString, OutEndpoint);

		if (bSupportHostname)
		{
			bParsedEndpoint = FIPv4Endpoint::FromHostAndPort(InEndpointString, OutEndpoint);
		}

		return bParsedEndpoint;
	}

    /** Initializes the message bridge with the current settings. */
    void InitializeBridge()
    {
        ShutdownBridge();
    	
        UQuicMessagingSettings* Settings = GetMutableDefault<UQuicMessagingSettings>();
        bool bResaveSettings = false;

		FIPv4Endpoint UnicastEndpoint;

		if (!ParseEndpoint(Settings->UnicastEndpoint, UnicastEndpoint))
		{
			if (!Settings->UnicastEndpoint.IsEmpty())
			{
				UE_LOGF(LogQuicMessaging, Warning,
					"Invalid setting for UnicastEndpoint '%ls' - "
					"binding to all local network adapters instead",
					*Settings->UnicastEndpoint);
			}

			UnicastEndpoint = FIPv4Endpoint::Any;
			Settings->UnicastEndpoint = UnicastEndpoint.ToText().ToString();
			bResaveSettings = true;
		}

        // Initialize the service with the additional endpoints added through the modular interface
		TArray<FIPv4Endpoint> StaticEndpoints = AdditionalStaticEndpoints.Array();

		for (FString& StaticEndpoint : Settings->StaticEndpoints)
		{
			FIPv4Endpoint Endpoint;

			if (ParseEndpoint(StaticEndpoint, Endpoint))
			{
				StaticEndpoints.Add(Endpoint);
			}
			else
			{
				UE_LOGF(LogQuicMessaging, Warning,
					"Invalid QUIC Messaging Static Endpoint '%ls'", *StaticEndpoint);
			}
		}

        if (bResaveSettings)
        {
            Settings->SaveConfig();
        }
    	
		UE_LOGF(LogQuicMessaging, Log,
			"Initialized bridge on interface %ls", *UnicastEndpoint.ToString());

		const TSharedRef<FQuicEndpointConfig> EndpointConfig = (Settings->bIsClient)
			? GetClientConfig(Settings) : GetServerConfig(Settings);

		EndpointConfig->Endpoint = UnicastEndpoint;
		EndpointConfig->LocalNodeId = Settings->EndpointGuid;
		EndpointConfig->DiscoveryTimeoutSec = Settings->DiscoveryTimeoutSeconds;

		EndpointConfig->EncryptionMode = (Settings->bEncryption)
			? EEncryptionMode::Enabled : EEncryptionMode::Disabled;

        const TSharedRef<FQuicMessageTransport, ESPMode::ThreadSafe> Transport =
			MakeShared<FQuicMessageTransport, ESPMode::ThreadSafe>(
				Settings->bIsClient, EndpointConfig, MoveTemp(StaticEndpoints));

        Transport->OnTransportError().BindRaw(
			this, &FQuicMessagingModule::HandleTransportError);

        WeakBridgeTransport = Transport;
        MessageBridge = FMessageBridgeBuilder().UsingTransport(Transport);
    }

    /**
	 * Parse command line arguments to override QuicMessagingSettings.
	 */
	void ParseCommandLine(UQuicMessagingSettings* Settings, const TCHAR* CommandLine) const
	{
		if (Settings && CommandLine)
		{
			// Parse value overrides (if present)
			FParse::Bool(CommandLine, 
				TEXT("-QUICMESSAGING_TRANSPORT_ENABLE="),
				Settings->EnableTransport);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_TRANSPORT_UNICAST="),
				Settings->UnicastEndpoint);

			FParse::Bool(CommandLine,
				TEXT("-QUICMESSAGING_TRANSPORT_ISCLIENT="),
				Settings->bIsClient);

			FParse::Bool(CommandLine,
				TEXT("-QUICMESSAGING_TRANSPORT_ENCRYPTION="),
				Settings->bEncryption);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_DISCOVERY_TIMEOUT_SEC="),
				Settings->DiscoveryTimeoutSeconds);

			FParse::Bool(CommandLine,
				TEXT("-QUICMESSAGING_AUTH_SERVER_ENABLED="),
				Settings->bAuthEnabled);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_MAX_AUTH_MESSAGE_SIZE_BYTES="),
				Settings->MaxAuthenticationMessageSize);

			FParse::Bool(CommandLine,
				TEXT("-QUICMESSAGING_VERIFY_CERTIFICATE="),
				Settings->bClientVerificationEnabled);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_SERVER_CERTIFICATE="),
				Settings->QuicServerCertificate);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_SERVER_PRIVATEKEY="),
				Settings->QuicServerPrivateKey);

			FParse::Bool(CommandLine,
				TEXT("-QUICMESSAGING_CONN_COOLDOWN_ENABLED="),
				Settings->bConnectionCooldownEnabled);

			FParse::Value(CommandLine, 
				TEXT("-QUICMESSAGING_CONN_COOLDOWN_MAX_ATTEMPTS="),
				Settings->ConnectionCooldownMaxAttempts);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_CONN_COOLDOWN_PERIOD_SEC="),
				Settings->ConnectionCooldownPeriodSeconds);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_CONN_COOLDOWN_SEC="),
				Settings->ConnectionCooldownSeconds);

			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_CONN_COOLDOWN_MAX_SEC="),
				Settings->ConnectionCooldownMaxSeconds);

			FString StaticEndpoints;
			FParse::Value(CommandLine,
				TEXT("-QUICMESSAGING_TRANSPORT_STATIC="), StaticEndpoints, false);

			TArray<FString> CommandLineStaticEndpoints;
			StaticEndpoints.ParseIntoArray(CommandLineStaticEndpoints, TEXT(","));

			for (const FString& CmdStaticEndpoint : CommandLineStaticEndpoints)
			{
				Settings->StaticEndpoints.AddUnique(CmdStaticEndpoint);
			}
		}
	}

	/** Get the config for client mode. */
	TSharedRef<FQuicEndpointConfig> GetClientConfig(const UQuicMessagingSettings* InSettings)
	{
		FQuicClientConfig ClientEndpointConfig;

		if (!InSettings)
		{
			UE_LOGF(LogQuicMessaging, Fatal,
				"[QuicMessagingModule::GetClientConfig] Pointer to settings not valid.");
		}

		ClientEndpointConfig.ClientVerificationMode = (InSettings->bClientVerificationEnabled)
			? EQuicClientVerificationMode::Verify : EQuicClientVerificationMode::Pass;


		return MakeShared<FQuicClientConfig>(ClientEndpointConfig);
	}

	/** Get the config for server mode. */
	TSharedRef<FQuicEndpointConfig> GetServerConfig(const UQuicMessagingSettings* InSettings)
	{
		FQuicServerConfig ServerEndpointConfig;

		if (!InSettings)
		{
			UE_LOGF(LogQuicMessaging, Fatal,
				"[QuicMessagingModule::GetServerConfig] Pointer to settings not valid.");
		}

		ServerEndpointConfig.Certificate = InSettings->QuicServerCertificate;
		ServerEndpointConfig.PrivateKey = InSettings->QuicServerPrivateKey;

		ServerEndpointConfig.MaxAuthenticationMessageSize
			= InSettings->MaxAuthenticationMessageSize;

		ServerEndpointConfig.AuthenticationMode = (InSettings->bAuthEnabled)
			? EAuthenticationMode::Enabled : EAuthenticationMode::Disabled;

		ServerEndpointConfig.ConnCooldownMode = (InSettings->bConnectionCooldownEnabled)
			? EConnectionCooldownMode::Enabled : EConnectionCooldownMode::Disabled;

		ServerEndpointConfig.ConnCooldownMaxAttempts
			= InSettings->ConnectionCooldownMaxAttempts;

		ServerEndpointConfig.ConnCooldownPeriodSec
			= InSettings->ConnectionCooldownPeriodSeconds;

		ServerEndpointConfig.ConnCooldownSec
			= InSettings->ConnectionCooldownSeconds;

		ServerEndpointConfig.ConnCooldownMaxSec
			= InSettings->ConnectionCooldownMaxSeconds;

		return MakeShared<FQuicServerConfig>(ServerEndpointConfig);
	}

    /** Shuts down the message bridge. */
    void ShutdownBridge()
    {
        StopAutoRepairRoutine();

        if (WeakBridgeTransport.IsValid())
        {
			WeakBridgeTransport.Reset();
        }

        if (MessageBridge.IsValid())
        {
            MessageBridge->Disable();
        	
            FPlatformProcess::Sleep(0.2f);

			if (MessageBridge.IsValid())
			{
				MessageBridge.Reset();
			}
        }
    }

private:

	void CreateEndpointGuid() const
	{
		UQuicMessagingSettings* Settings = GetMutableDefault<UQuicMessagingSettings>();
		Settings->EndpointGuid = FGuid::NewGuid();
		Settings->SaveConfig();
	}

    /** Callback for when the application has been reactivated. */
    void HandleApplicationHasReactivated()
	{
		RestartServices();
	}

	/** Callback for when the application will be deactivated.*/
	void HandleApplicationWillDeactivate()
	{
		ShutdownServices();
	}

    /** Callback for when the settings were saved. */
    bool HandleSettingsSaved()
    {
		UE_LOGF(LogQuicMessaging, Display,
			"QUIC settings saved, restarting services...");

        RestartServices();

        return true;
    }

    void HandleAppPreExit() const
    {
        if (TSharedPtr<FQuicMessageTransport> QuicTransport = WeakBridgeTransport.Pin())
        {
            QuicTransport->OnAppPreExit();
        }
    }

    void HandleTransportError()
    {
        const UQuicMessagingSettings* Settings = GetDefault<UQuicMessagingSettings>();
        if (Settings->bAutoRepair)
        {
            StartAutoRepairRoutine(Settings->AutoRepairAttemptLimit);
        }
        else
        {
			UE_LOGF(LogQuicMessaging, Error,
				"QUIC messaging encountered an error. "
				"Please restart the service for proper functionality");
        }
    }

    void StartAutoRepairRoutine(uint32 MaxRetryAttempt)
    {
        StopAutoRepairRoutine();

        FTimespan CheckDelay(0, 0, 1);
        uint32 CheckNumber = 1;

        AutoRepairHandle = FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda(
				[WeakTransport = WeakBridgeTransport, LastTime = FDateTime::UtcNow(),
				CheckDelay, CheckNumber, MaxRetryAttempt](float DeltaTime) mutable
        {
            QUICK_SCOPE_CYCLE_COUNTER(STAT_FQuicMessagingModule_AutoRepair);

			bool bContinue = true;
            const FDateTime UtcNow = FDateTime::UtcNow();

			if (LastTime + (CheckDelay * CheckNumber) <= UtcNow)
			{
				if (const TSharedPtr<FQuicMessageTransport> Transport = WeakTransport.Pin())
				{
					// if the restart fail, continue the routine if we are still under the retry attempt limit
					bContinue = !Transport->RestartTransport();
					bContinue = bContinue && CheckNumber <= MaxRetryAttempt;
				}
				// if we do not have a valid transport also stop the routine
				else
				{
					bContinue = false;
				}
				++CheckNumber;
				LastTime = UtcNow;
			}
			return bContinue;
			
        }), 1.0f);

		UE_LOGF(LogQuicMessaging, Warning,
			"QUIC messaging encountered an error. "
			"Auto repair routine started for reinitialization");
    }

    void StopAutoRepairRoutine() const
    {
        if (AutoRepairHandle.IsValid())
        {
			FTSTicker::RemoveTicker(AutoRepairHandle);
        }
    }

private:

    /** Name of the network messaging extension. */
    static FName QuicMessagingName;

    /** Holds the message bridge if present. */
	TSharedPtr<IMessageBridge, ESPMode::ThreadSafe> MessageBridge = nullptr;

	/** Holds the bridge transport if present.  */
	TWeakPtr<FQuicMessageTransport, ESPMode::ThreadSafe> WeakBridgeTransport = nullptr;

	/** Critical section protecting access to the transport static endpoints and additional static endpoints. */
	FCriticalSection StaticEndpointsCS;

	/** Holds additional static endpoints added through the modular feature interface. */
	TSet<FIPv4Endpoint> AdditionalStaticEndpoints = TSet<FIPv4Endpoint>();

	/** Holds the delegate handle for the auto repair routine. */
	FTSTicker::FDelegateHandle AutoRepairHandle = nullptr;
};

FName FQuicMessagingModule::QuicMessagingName("QuicMessaging");


void EmptyLinkFunctionForStaticInitializationQuicMessagingTests()
{
	// Force references to the object files containing the functions below, to prevent them being
	// excluded by the linker when the plug-in is compiled into a static library for monolithic builds.

	extern void EmptyLinkFunctionForStaticInitializationQuicMessageTransportTest();
	EmptyLinkFunctionForStaticInitializationQuicMessageTransportTest();
}


IMPLEMENT_MODULE(FQuicMessagingModule, QuicMessaging);


#undef LOCTEXT_NAMESPACE
