// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Interfaces/IPv4/IPv4Address.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"


/** Declares a log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogQuicMessaging, VeryVerbose, All);


/** Defines the maximum number of annotations a message can have. */
#define QUIC_MESSAGING_MAX_ANNOTATIONS 128

/** Defines the maximum number of recipients a message can have. */
#define QUIC_MESSAGING_MAX_RECIPIENTS 1024


/** Defines the MsQuic version to be used. */
static FString MSQUIC_VERSION = "v220";

/** Defines the MsQuic binaries path. */
static FString MSQUIC_BINARIES_PATH = "Binaries/ThirdParty/MsQuic/" + MSQUIC_VERSION;

