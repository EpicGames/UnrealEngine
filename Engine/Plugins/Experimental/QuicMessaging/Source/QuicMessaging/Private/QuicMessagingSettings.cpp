// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/QuicMessagingSettings.h"


UQuicMessagingSettings::UQuicMessagingSettings(const FObjectInitializer& ObjectInitializer)
    : Super(ObjectInitializer)
	, EnableTransport(true)
{ }