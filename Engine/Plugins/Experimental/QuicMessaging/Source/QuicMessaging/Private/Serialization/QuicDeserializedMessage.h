// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "IMessageContext.h"
#include "Misc/DateTime.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "QuicMessages.h"


class IMessageAttachment;
class UScriptStruct;


/**
 * Enumerates possibly states of a deserialized message.
 */
enum class EQuicDeserializedMessageState
{
    /** The message data is complete. */
	Complete,

	/** The message data is incomplete. */
	Incomplete,

	/** The message data is invalid. */
	Invalid
};


/**
 * Holds deserialized message data.
 */
class FQuicDeserializedMessage
	: public IMessageContext
{

	friend class FQuicDeserializeMessageTask;

public:

	/** Default constructor. */
	FQuicDeserializedMessage() = delete;

	/**
	 * Create and initialize a new instance.
	 */
	FQuicDeserializedMessage(FInboundMessage InInboundMessage)
		: State(EQuicDeserializedMessageState::Incomplete)
		, InboundMessage(InInboundMessage)
		, Annotations(TMap<FName, FString>())
		, Attachment(nullptr)
		, Expiration(FDateTime::UtcNow())
		, MessageData(nullptr)
		, Recipients(TArray<FMessageAddress>())
		, Scope(EMessageScope::All)
		, Flags(EMessageFlags::None)
		, Sender(FMessageAddress())
		, TimeSent(FDateTime::UtcNow())
		, TypeInfo(nullptr)
	{ }

	/** Virtual destructor. */
	virtual ~FQuicDeserializedMessage() override
	{
		if (MessageData != nullptr)
		{
			if (TypeInfo.IsValid())
			{
				TypeInfo->DestroyStruct(MessageData);
			}

			FMemory::Free(MessageData);
			MessageData = nullptr;
		}
	}

public:

	/**
	 * Gets the state of the message data.
	 */
	EQuicDeserializedMessageState GetState() const
	{
		return State;
	}

	/**
	 * Updates the state of this message data.
	 */
	void UpdateState(EQuicDeserializedMessageState InState)
	{
		State = InState;
	}

	/**
	 * Get the inbound message.
	 */
	FInboundMessage& GetInboundMessage()
	{
		return InboundMessage;
	}

public:

	//~ IMessageContext interface

	virtual const TMap<FName, FString>& GetAnnotations() const override
	{
		return Annotations;
	}

	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const override
	{
		return Attachment;
	}

	virtual const FDateTime& GetExpiration() const override
	{
		return Expiration;
	}

	virtual const void* GetMessage() const override
	{
		return MessageData;
	}

	virtual const TWeakObjectPtr<UScriptStruct>& GetMessageTypeInfo() const override
	{
		return TypeInfo;
	}

	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const override
	{
		return nullptr;
	}

	virtual const TArray<FMessageAddress>& GetRecipients() const override
	{
		return Recipients;
	}

	virtual EMessageScope GetScope() const override
	{
		return Scope;
	}

	virtual EMessageFlags GetFlags() const override
	{
		return Flags;
	}

	virtual const FMessageAddress& GetSender() const override
	{
		return Sender;
	}

	virtual const FMessageAddress& GetForwarder() const override
	{
		return Sender;
	}

	virtual ENamedThreads::Type GetSenderThread() const override
	{
		return ENamedThreads::AnyThread;
	}

	virtual const FDateTime& GetTimeForwarded() const override
	{
		return TimeSent;
	}

	virtual const FDateTime& GetTimeSent() const override
	{
		return TimeSent;
	}

private:

	/** Holds the deserialized message state. */
	EQuicDeserializedMessageState State;

	/** Holds the inbound message. */
	FInboundMessage InboundMessage;

private:

	/** Holds the optional message annotations. */
	TMap<FName, FString> Annotations;

	/** Holds a pointer to attached binary data. */
	TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> Attachment;

	/** Holds the expiration time. */
	FDateTime Expiration;

	/** Holds the message. */
	void* MessageData;

	/** Holds the message recipients. */
	TArray<FMessageAddress> Recipients;

	/** Holds the message's scope. */
	EMessageScope Scope;

	/** Holds the message's flags. */
	EMessageFlags Flags;

	/** Holds the sender's identifier. */
	FMessageAddress Sender;

	/** Holds the time at which the message was sent. */
	FDateTime TimeSent;

	/** Holds the message's type information. */
	TWeakObjectPtr<UScriptStruct> TypeInfo;

};




