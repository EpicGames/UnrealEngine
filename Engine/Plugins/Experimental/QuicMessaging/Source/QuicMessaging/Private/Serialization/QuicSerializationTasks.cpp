// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicSerializationTasks.h"
#include "QuicMessagingPrivate.h"

#include "HAL/Event.h"
#include "IMessageContext.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"
#include "StructSerializer.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "StructDeserializer.h"

#include "QuicMessagingSettings.h"
#include "QuicSerializedMessage.h"
#include "QuicDeserializedMessage.h"


/**
 * FQuicSerializeMessageTask
 */

void FQuicSerializeMessageTask::SerializeMessage(FArchive& Archive, EQuicMessageFormat MessageFormat,
	const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext,
	const EStructSerializerBackendFlags StructSerializerBackendFlags)
{
	const FName& MessageType = InMessageContext->GetMessageTypePathName().GetAssetName();
	Archive << const_cast<FName&>(MessageType);

	const FMessageAddress& Sender = InMessageContext->GetSender();
	Archive << const_cast<FMessageAddress&>(Sender);

	const TArray<FMessageAddress>& Recipients = InMessageContext->GetRecipients();
	Archive << const_cast<TArray<FMessageAddress>&>(Recipients);

	EMessageScope Scope = InMessageContext->GetScope();
	Archive << Scope;

	EMessageFlags Flags = InMessageContext->GetFlags();
	Archive << Flags;

	const FDateTime& TimeSent = InMessageContext->GetTimeSent();
	Archive << const_cast<FDateTime&>(TimeSent);

	const FDateTime& Expiration = InMessageContext->GetExpiration();
	Archive << const_cast<FDateTime&>(Expiration);

	int32 NumAnnotations = InMessageContext->GetAnnotations().Num();
	Archive << NumAnnotations;

	for (const auto& AnnotationPair : InMessageContext->GetAnnotations())
	{
		Archive << const_cast<FName&>(AnnotationPair.Key);
		Archive << const_cast<FString&>(AnnotationPair.Value);
	}

	// Message Wire Format Id
	check(MessageFormat == EQuicMessageFormat::CborPlatformEndianness
		|| MessageFormat == EQuicMessageFormat::CborStandardEndianness);
	
	uint8 Format = (uint8)(MessageFormat);
	Archive << Format;

	// Serialize message body with cbor
	FCborStructSerializerBackend Backend(Archive,
		StructSerializerBackendFlags | (MessageFormat == EQuicMessageFormat::CborStandardEndianness
		? EStructSerializerBackendFlags::WriteCborStandardEndianness : EStructSerializerBackendFlags::None));
	
	FStructSerializer::Serialize(InMessageContext->GetMessage(),
        *InMessageContext->GetMessageTypeInfo(), Backend);
}


void FQuicSerializeMessageTask::DoTask(ENamedThreads::Type CurrentThread,
    const FGraphEventRef& MyCompletionGraphEvent)
{
    if (MessageContext->IsValid())
    {
		FMemoryWriter MemoryWriter(*SerializedMessage->GetDataPtr(), true);

        FArchive& Archive = MemoryWriter;

        SerializeMessage(Archive, SerializedMessage->GetFormat(), MessageContext,
            EStructSerializerBackendFlags::Default);
        
        SerializedMessage->UpdateState(EQuicSerializedMessageState::Complete);
    }
    else
    {
        SerializedMessage->UpdateState(EQuicSerializedMessageState::Invalid);
    }

    // Signal task completion
	TSharedPtr<FEvent, ESPMode::ThreadSafe> CompletionEvent = CompletionEventPtr.Pin();

	if (CompletionEvent.IsValid())
	{
		CompletionEvent->Trigger();
	}
}


ENamedThreads::Type FQuicSerializeMessageTask::GetDesiredThread()
{
	return ENamedThreads::AnyThread;
}


TStatId FQuicSerializeMessageTask::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FQuicSerializeMessageTask, STATGROUP_TaskGraphTasks);
}


ESubsequentsMode::Type FQuicSerializeMessageTask::GetSubsequentsMode()
{
	return ESubsequentsMode::FireAndForget;
}


/**
 * FQuicDeserializeMessageTask
 */

bool FQuicDeserializeMessageTask::DeserializeMessage(FMemoryReader& MessageReader)
{

    // message type info
	{
		FName MessageType;
		MessageReader << MessageType;

		// @todo gmp: cache message types for faster lookup
		DeserializedMessage->TypeInfo = FindFirstObjectSafe<UScriptStruct>(*MessageType.ToString(), EFindFirstObjectOptions::EnsureIfAmbiguous);

		if (!DeserializedMessage->TypeInfo.IsValid(false, true))
		{
			return false;
		}
	}

	// Sender address
	{
		MessageReader << DeserializedMessage->Sender;
	}

	// Recipient addresses
	{
		int32 NumRecipients = 0;
		MessageReader << NumRecipients;

		if ((NumRecipients < 0) || (NumRecipients > QUIC_MESSAGING_MAX_RECIPIENTS))
		{
			return false;
		}

		DeserializedMessage->Recipients.Empty(NumRecipients);

		while (0 < NumRecipients--)
		{
			MessageReader << DeserializedMessage->Recipients.AddDefaulted_GetRef();
		}
	}

	// Message scope
	{
		MessageReader << DeserializedMessage->Scope;

		if (DeserializedMessage->Scope > EMessageScope::All)
		{
			return false;
		}
	}

	// Message flags
	{
		MessageReader << DeserializedMessage->Flags;
	}

	// Time sent & expiration
	{
		MessageReader << DeserializedMessage->TimeSent;
		MessageReader << DeserializedMessage->Expiration;
	}

	// Annotations
	{
		int32 NumAnnotations = 0;
		MessageReader << NumAnnotations;

		if (NumAnnotations > QUIC_MESSAGING_MAX_ANNOTATIONS)
		{
			return false;
		}

		while (0 < NumAnnotations--)
		{
			FName Key;
			FString Value;

			MessageReader << Key;
			MessageReader << Value;

			DeserializedMessage->Annotations.Add(Key, Value);
		}
	}

	// Wire format 
	uint8 FormatId;
	MessageReader << FormatId;
	EQuicMessageFormat MessageFormat = (EQuicMessageFormat)FormatId;

	// Create message body
	DeserializedMessage->MessageData = FMemory::Malloc(DeserializedMessage->TypeInfo->GetStructureSize());
	DeserializedMessage->TypeInfo->InitializeStruct(DeserializedMessage->MessageData);

	switch (MessageFormat)
	{

	case EQuicMessageFormat::Json:
	{
		// Deserialize json
		FJsonStructDeserializerBackend Backend(MessageReader);
		bool DeserializeResult = FStructDeserializer::Deserialize(DeserializedMessage->MessageData, *DeserializedMessage->TypeInfo, Backend);

		if (!DeserializeResult)
		{
			UE_LOGF(LogQuicMessaging, Error, "[DeserializeMessage] Json deserialize result failed");
		}

		return DeserializeResult;
	}

	case EQuicMessageFormat::CborPlatformEndianness:
	{
		// Deserialize cbor (using this platform endianness)
		FCborStructDeserializerBackend Backend(MessageReader, ECborEndianness::Platform);
		bool DeserializeResult = FStructDeserializer::Deserialize(DeserializedMessage->MessageData, *DeserializedMessage->TypeInfo, Backend);

		if (!DeserializeResult)
		{
			UE_LOGF(LogQuicMessaging, Error, "[DeserializeMessage] CborPlatform deserialize result failed");
		}

		return DeserializeResult;
	}

	case EQuicMessageFormat::CborStandardEndianness:
	{
		// Deserialize cbor (using the CBOR standard endianness - big endian)
		FCborStructDeserializerBackend Backend(MessageReader, ECborEndianness::StandardCompliant);
		bool DeserializeResult = FStructDeserializer::Deserialize(DeserializedMessage->MessageData, *DeserializedMessage->TypeInfo, Backend);

		if (!DeserializeResult)
		{
			UE_LOGF(LogQuicMessaging, Error, "[DeserializeMessage] CborStandard deserialize result failed");
		}

		return DeserializeResult;
	}

	case EQuicMessageFormat::TaggedProperty:
	{
		// Deserialize message body using tagged property
		// Hack: this binary serialization should use a more standard protocol, should use cbor
		DeserializedMessage->TypeInfo->SerializeItem(MessageReader, DeserializedMessage->MessageData, nullptr);
		bool DeserializeResult = !MessageReader.GetError();

		if (!DeserializeResult)
		{
			UE_LOGF(LogQuicMessaging, Error, "[DeserializeMessage] TaggedProperty deserialize result failed");
		}

		return DeserializeResult;
	}

	default:

		// Unsupported format
		return false;
		
	}
}


void FQuicDeserializeMessageTask::DoTask(ENamedThreads::Type CurrentThread,
    const FGraphEventRef& MyCompletionGraphEvent)
{
    if (Data.IsValid())
    {
		FMemoryReader MessageReader(*Data.Get(), true);
		MessageReader.ArMaxSerializeSize = NAME_SIZE;

		bool DeserializeMessageResult = DeserializeMessage(MessageReader);

		if (DeserializeMessageResult)
		{
			DeserializedMessage->UpdateState(EQuicDeserializedMessageState::Complete);
		}
		else
		{
			DeserializedMessage->UpdateState(EQuicDeserializedMessageState::Invalid);
		}

		Data.Reset();
    }
    else
    {
        DeserializedMessage->UpdateState(EQuicDeserializedMessageState::Invalid);
    }

    // signal task completion
	TSharedPtr<FEvent, ESPMode::ThreadSafe> CompletionEvent = CompletionEventPtr.Pin();

	if (CompletionEvent.IsValid())
	{
		CompletionEvent->Trigger();
	}
}


ENamedThreads::Type FQuicDeserializeMessageTask::GetDesiredThread()
{
	return ENamedThreads::AnyThread;
}


TStatId FQuicDeserializeMessageTask::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FQuicDeserializeMessageTask, STATGROUP_TaskGraphTasks);
}


ESubsequentsMode::Type FQuicDeserializeMessageTask::GetSubsequentsMode()
{
	return ESubsequentsMode::FireAndForget;
}




