// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"


class FEvent;
class FMemoryReader;
class FQuicSerializedMessage;
class FQuicDeserializedMessage;
class IMessageContext;
enum class EStructSerializerBackendFlags;
enum class EQuicMessageFormat : uint8;


class FQuicSerializeMessageTask
{

public:

	FQuicSerializeMessageTask() = delete;

	/**
	 * Creates a new serialize message task.
	 *
	 * @param InMessageContext The message context
	 * @param InSerializedMessage The serialized message
	 * @param InCompletionEvent The completion event
	 */
	FQuicSerializeMessageTask(
		TSharedRef<IMessageContext, ESPMode::ThreadSafe> InMessageContext,
		TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe> InSerializedMessage,
		TSharedPtr<FEvent, ESPMode::ThreadSafe> InCompletionEvent)
		: MessageContext(InMessageContext)
		, SerializedMessage(InSerializedMessage)
		, CompletionEventPtr(InCompletionEvent)
	{ }

private:

	/**
	 * Serializes the message.
	 *
	 * @param Archive The archive where data will be stored
	 * @param MessageFormat The message format
	 * @param InMessageContext Shared ref of the message context
	 * @param StructSerializerBackendFlags Flags for the struct serializer backend
	 */
	void SerializeMessage(FArchive& Archive, EQuicMessageFormat MessageFormat,
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& InMessageContext,
		const EStructSerializerBackendFlags StructSerializerBackendFlags);

public:

	/**
	 * Performs the actual task.
	 *
	 * @param CurrentThread The thread that this task is executing on
	 * @param MyCompletionGraphEvent The completion event
	 */
	void DoTask(ENamedThreads::Type CurrentThread,
		const FGraphEventRef& MyCompletionGraphEvent);

	/**
	 * Returns the name of the thread that this task should run on.
	 *
	 * @return The thread name
	 */
	ENamedThreads::Type GetDesiredThread();

	/**
	 * Gets the task's stats tracking identifier.
	 *
	 * @return Stats identifier
	 */
	TStatId GetStatId() const;

	/**
	 * Gets the mode for tracking subsequent tasks.
	 *
	 * @return Always track subsequent tasks
	 */
	static ESubsequentsMode::Type GetSubsequentsMode();

private:

	/** The context of the message to serialize. */
	TSharedRef<IMessageContext, ESPMode::ThreadSafe> MessageContext;

	/** A reference to the serialized message data. */
	TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe> SerializedMessage;

	/** An event signaling that the message was serialized. */
	TWeakPtr<FEvent, ESPMode::ThreadSafe> CompletionEventPtr;

};


class FQuicDeserializeMessageTask
{

public:

	FQuicDeserializeMessageTask() = delete;

	/**
	 * Creates a new deserialize message task.
	 *
	 * @param InData The data
	 * @param InDeserializedMessage The deserialized message
	 * @param InCompletionEvent The completion event
	 */
	FQuicDeserializeMessageTask(
		TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> InData,
		TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe> InDeserializedMessage,
		TSharedPtr<FEvent, ESPMode::ThreadSafe> InCompletionEvent)
		: Data(InData)
		, DeserializedMessage(InDeserializedMessage)
		, CompletionEventPtr(InCompletionEvent)
	{ }

private:

	/**
	 * Deserializes the message.
	 *
	 * @param MessageReader The already initialized message reader
	 */
	bool DeserializeMessage(FMemoryReader& MessageReader);

public:

	/**
	 * Performs the actual task.
	 *
	 * @param CurrentThread The thread that this task is executing on
	 * @param MyCompletionGraphEvent The completion event
	 */
	void DoTask(ENamedThreads::Type CurrentThread,
		const FGraphEventRef& MyCompletionGraphEvent);

	/**
	 * Returns the name of the thread that this task should run on.
	 *
	 * @return The thread name
	 */
	ENamedThreads::Type GetDesiredThread();

	/**
	 * Gets the task's stats tracking identifier.
	 *
	 * @return Stats identifier
	 */
	TStatId GetStatId() const;

	/**
	 * Gets the mode for tracking subsequent tasks.
	 *
	 * @return Always track subsequent tasks
	 */
	static ESubsequentsMode::Type GetSubsequentsMode();

private:

	/** A pointer to the unserialized message data. */
	TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> Data;

	/** A pointer to the deserialized message. */
	TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage;

	/** An event signaling that the message was serialized. */
	TWeakPtr<FEvent, ESPMode::ThreadSafe> CompletionEventPtr;

};
