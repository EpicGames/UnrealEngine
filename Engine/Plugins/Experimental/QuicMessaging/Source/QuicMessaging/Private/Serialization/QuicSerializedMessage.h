// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "IMessageContext.h"

#include "QuicMessagingSettings.h"
#include "QuicFlags.h"


/**
 * Enumerates possibly states of a serialized message.
 */
enum class EQuicSerializedMessageState
{
    /** The message data is complete. */
	Complete,

	/** The message data is incomplete. */
	Incomplete,

	/** The message data is invalid. */
	Invalid
};


/**
 * Holds serialized message data.
 */
class FQuicSerializedMessage
{

public:

	FQuicSerializedMessage() = delete;

    /** Default constructor */
	FQuicSerializedMessage(EQuicMessageFormat InMessageFormat, EMessageFlags InFlags,
		TArray<FGuid> InRecipients, EQuicMessageType InMessageType,
		TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> InDataPtr,
		TSharedRef<IMessageContext, ESPMode::ThreadSafe> InMessageContext)
		: DataPtr(InDataPtr)
		, MessageContext(InMessageContext)
		, State(EQuicSerializedMessageState::Incomplete)
		, Flags(InFlags)
		, Format(InMessageFormat)
		, Recipients(InRecipients)
		, MessageType(InMessageType)
	{ }

public:

	/**
	 * Get the pointer to the data.
	 */
	TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> GetDataPtr()
	{
		return DataPtr;
	}

	/**
	 * Get the message context.
	 */
	TSharedRef<IMessageContext, ESPMode::ThreadSafe> GetMessageContext()
	{
		return MessageContext;
	}

	/**
	 * Gets the state of the message data.
	 */
	EQuicSerializedMessageState GetState() const
	{
		return State;
	}

	/**
	 * Updates the state of this message data.
	 */
	void UpdateState(EQuicSerializedMessageState InState)
	{
		State = InState;
	}

	/**
	 * Get the message flags.
	 */
	EMessageFlags GetFlags() const
	{
		return Flags;
	}

	/**
	 * Get the format used to encode the message
	 */
	EQuicMessageFormat GetFormat() const
	{
		return Format;
	}

	/**
	 * Get the message recipients.
	 */
	TArray<FGuid> GetRecipients() const
	{
		return Recipients;
	}

	/**
	 * Get the message type.
	 */
	EQuicMessageType GetMessageType() const
	{
		return MessageType;
	}

private:

	/** Holds the serialized data. */
	TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe> DataPtr;

	/** Holds the message context. */
	TSharedRef<IMessageContext, ESPMode::ThreadSafe> MessageContext;

	/** Holds the message data state. */
	EQuicSerializedMessageState State;

	/** Holds message flags, captured from context. */
	EMessageFlags Flags;

	/** The format used to serialize the message. */
	EQuicMessageFormat Format;

	/** Holds the message recipients. */
	TArray<FGuid> Recipients;

	/** Holds the message type. */
	EQuicMessageType MessageType;

};




