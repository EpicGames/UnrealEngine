// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/AutomationTest.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformProcess.h"
#include "IMessageContext.h"
#include "IMessageTransportHandler.h"

#include "Tests/QuicMessagingTestTypes.h"
#include "Transport/QuicMessageTransport.h"
#include "IQuicNetworkMessagingExtension.h"
#include "QuicEndpointConfig.h"
#include "QuicTransportMessages.h"
#include "Shared/QuicMessagingSettings.h"
#include "Features/IModularFeatures.h"
#include "MessageEndpoint.h"


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FQuicMessageTransportTest,
	"System.Core.Messaging.Transports.Quic.QuicMessageTransport (may take some minutes!)",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)


/**
 * QuicMessaging test utilities
 */
namespace FQuicTestUtils
{

	static IQuicNetworkMessagingExtension* GetQuicFeature()
	{
		TArray<INetworkMessagingExtension*> Features = IModularFeatures::Get()
			.GetModularFeatureImplementations<INetworkMessagingExtension>(
				INetworkMessagingExtension::ModularFeatureName);

		for (const auto Feature : Features)
		{
			if (Feature->GetName() == "QuicMessaging")
			{
				IQuicNetworkMessagingExtension* QuicFeature
					= static_cast<IQuicNetworkMessagingExtension*>(Feature);

				return QuicFeature;
			}
		}

		return nullptr;
	}

	template<typename T>
	static TSharedRef<FQuicMetaMessageContext> WrapMessage(T* Message)
	{
		TSharedRef<FQuicMetaMessageContext> Context
			= MakeShared<FQuicMetaMessageContext>(Message);

		return Context;
	}

	static FString GetAuthPayload()
	{
		return TEXT("1234567890");
	}

};


/**
 * Test state
 */
class FQuicMessageTransportTestState
    : public IMessageTransportHandler
{
    
public:

    FQuicMessageTransportTestState(
        FAutomationTestBase& Test, const bool bInIsClient,
        const TSharedRef<FQuicEndpointConfig> InEndpointConfig,
		const bool bInEchoEndpoint, TArray<FIPv4Endpoint> InStaticEndpoints)
		: bEchoEndpoint(bInEchoEndpoint)
		, Transport(MakeShared<FQuicMessageTransport, ESPMode::ThreadSafe>(
			bInIsClient, InEndpointConfig, MoveTemp(InStaticEndpoints)))
    {
    }

    virtual ~FQuicMessageTransportTestState()
	{
	}

public:

    bool Start()
    {
		Transport->OnMetaMessageReceived().AddRaw(
			this, &FQuicMessageTransportTestState::OnQuicMetaMessageReceived);

		return Transport->StartTransport(*this);
    }

	void Stop() const
	{
		Transport->OnMetaMessageReceived().RemoveAll(this);

		Transport->StopTransport();
	}

public:

    void AddClient(const FIPv4Endpoint& RemoteEndpoint) const
    {
        Transport->AddStaticEndpoint(RemoteEndpoint);
    }

    void RemoveClient(const FIPv4Endpoint& RemoteEndpoint) const
    {
        Transport->RemoveStaticEndpoint(RemoteEndpoint);
    }

	void SetNodeAuthenticated(const FGuid& NodeId) const
    {
		Transport->SetNodeAuthenticated(NodeId);
    }

    void BroadcastMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) const
	{
		const TArray<FGuid> Recipients;

		Transport->TransportMessage(Context, Recipients);
	}

	void SendMessage(const FGuid& NodeId,
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context) const
	{
    	TArray<FGuid> Recipients;
		Recipients.Add(NodeId);

		Transport->TransportMessage(Context, Recipients);
	}

	void SendAuthMessage(const FGuid& NodeId, const FString& Payload) const
    {
		FQuicAuthMessage* AuthMessage
			= FMessageEndpoint::MakeMessage<FQuicAuthMessage>();

		AuthMessage->Payload = Payload;

		const TSharedRef<FQuicMetaMessageContext> Context
			= FQuicTestUtils::WrapMessage(AuthMessage);

		Transport->TransportAuthMessage(Context, NodeId);
    }

	void TransportAuthMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
		const FGuid& Recipient) const
    {
		Transport->TransportAuthMessage(Context, Recipient);
    }

private:

	void OnQuicAuthMessageReceived(
		const FGuid LocalNodeId, const TSharedRef<IMessageContext>& Context)
	{
		const FQuicAuthMessage* AuthMessage
			= static_cast<const FQuicAuthMessage*>(Context->GetMessage());

		if (!AuthMessage)
		{
			UE_LOGF(LogTemp, Error,
				"[QuicMessageTest::OnQuicMetaMessageReceived] "
					"AuthMessage ptr not valid.");

			return;
		}

		const bool bAuthSuccessful
			= AuthMessage->Payload == FQuicTestUtils::GetAuthPayload();

		// Set the node as authenticated
		if (bAuthSuccessful)
		{
			SetNodeAuthenticated(LocalNodeId);
		}

		FQuicAuthResponseMessage* AuthResponseMessage
			= FMessageEndpoint::MakeMessage<FQuicAuthResponseMessage>();

		AuthResponseMessage->bAuthSuccessful = bAuthSuccessful;

		const TSharedRef<FQuicMetaMessageContext> ResponseContext
			= FQuicTestUtils::WrapMessage(AuthResponseMessage);

		Transport->TransportAuthResponseMessage(ResponseContext, LocalNodeId);
	}

	void OnQuicAuthResponseMessageReceived(
		const FGuid LocalNodeId, const TSharedRef<IMessageContext>& Context)
	{
		const FQuicAuthResponseMessage* AuthResponseMessage
			= static_cast<const FQuicAuthResponseMessage*>(Context->GetMessage());

		if (!AuthResponseMessage)
		{
			UE_LOGF(LogTemp, Error,
				"[QuicMessageTest::OnQuicMetaMessageReceived] "
					"AuthResponseMessage ptr not valid.");

			return;
		}

		bIsAuthenticated = AuthResponseMessage->bAuthSuccessful;

		if (!bIsAuthenticated)
		{
			UE_LOGF(LogTemp, Error,
				"[QuicMessageTest::OnQuicAuthResponseMessageReceived] "
					"Authentication failed. Reason: %ls", *AuthResponseMessage->Reason);
		}
	}

	void OnQuicMetaMessageReceived(
		const FGuid LocalNodeId, const TSharedRef<IMessageContext>& Context)
	{
		if (!Context->IsValid())
		{
			return;
		}

		if (Context->GetMessageTypeInfo()
			->IsChildOf(FQuicAuthMessage::StaticStruct()))
		{
			OnQuicAuthMessageReceived(LocalNodeId, Context);
		}
		else
		{
			OnQuicAuthResponseMessageReceived(LocalNodeId, Context);
		}
	}

public:

    TArray<FIPv4Endpoint> GetEndpoints() const
	{
    	return Transport->GetKnownEndpoints();
	}

	TOptional<FGuid> GetNodeId(const FString& RemoteEndpoint) const
    {
		FIPv4Endpoint ParsedRemoteEndpoint;

		if (!FIPv4Endpoint::Parse(RemoteEndpoint, ParsedRemoteEndpoint))
		{
			UE_LOGF(LogTemp, Error,
				"[QuicMessageTest::GetNodeId] Endpoint string could not be parsed.");

			return TOptional<FGuid>();
		}

		return Transport->GetNodeId(ParsedRemoteEndpoint);
    }

	const TArray<FGuid>& GetDiscoveredNodes() const
	{
		return DiscoveredNodes;
	}

	const TArray<FGuid> GetLostNodes() const
	{
		return LostNodes;
	}

	int32 GetNumReceivedMessages() const
	{
		return NumReceivedMessages;
	}

	bool IsAuthenticated() const
    {
		return bIsAuthenticated;
    }

public:

	//~ IMessageTransportHandler interface

	virtual void DiscoverTransportNode(const FGuid& NodeId) override
	{
		DiscoveredNodes.Add(NodeId);
	}

	virtual void ForgetTransportNode(const FGuid& NodeId) override
	{
		LostNodes.Add(NodeId);
	}

	virtual void ReceiveTransportMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
		const FGuid& NodeId) override
	{
		FPlatformAtomics::InterlockedIncrement(&NumReceivedMessages);

		if (bEchoEndpoint)
		{
			SendMessage(NodeId, Context);
		}
	}

private:

	bool bEchoEndpoint = false;

	bool bIsAuthenticated = false;

    TArray<FGuid> DiscoveredNodes;

	TArray<FGuid> LostNodes;

	int32 NumReceivedMessages = 0;

	TSharedRef<FQuicMessageTransport, ESPMode::ThreadSafe> Transport;

};


bool FQuicMessageTransportTest::RunTest(const FString& Parameters)
{
    const FIPv4Endpoint ServerEndpoint(FIPv4Address::Any, 9977);
	const FIPv4Endpoint ClientEndpoint(FIPv4Address(127, 0, 0, 1), 9977);

	const FGuid ServerGuid = FGuid::NewGuid();
	const FGuid ClientGuid = FGuid::NewGuid();

	const int32 NumTestMessages = 1000;

	const TArray<FIPv4Endpoint> StaticEndpoints;

    // Create the server transport config
    TSharedRef<FQuicServerConfig> ServerConfig = MakeShared<FQuicServerConfig>();

    {
        ServerConfig->Endpoint = ServerEndpoint;
        ServerConfig->LocalNodeId = ServerGuid;
        ServerConfig->EncryptionMode = EEncryptionMode::Enabled;
        ServerConfig->DiscoveryTimeoutSec = 0;
        ServerConfig->Certificate = "";
        ServerConfig->PrivateKey = "";
        ServerConfig->MaxAuthenticationMessageSize = 8 * 1024;
        ServerConfig->AuthenticationMode = EAuthenticationMode::Enabled;
        ServerConfig->ConnCooldownMode = EConnectionCooldownMode::Enabled;
        ServerConfig->ConnCooldownMaxAttempts = 5;
        ServerConfig->ConnCooldownPeriodSec = 30;
        ServerConfig->ConnCooldownSec = 30;
        ServerConfig->ConnCooldownMaxSec = 3600;
    }

    // Setup server transport
    const TSharedPtr<FQuicMessageTransportTestState> ServerTransport
        = MakeShared<FQuicMessageTransportTestState>(
            *this,                    // Test
			false,                    // bInIsClient
            ServerConfig,             // EndpointConfig
			true,                     // bInEchoEndpoint
            StaticEndpoints           // StaticEndpoints
        );

	// Create the client transport config
	TSharedRef<FQuicClientConfig> ClientConfig = MakeShared<FQuicClientConfig>();

	{
		ClientConfig->Endpoint = ClientEndpoint;
		ClientConfig->LocalNodeId = ClientGuid;
		ClientConfig->EncryptionMode = EEncryptionMode::Enabled;
		ClientConfig->DiscoveryTimeoutSec = 0;
		ClientConfig->ClientVerificationMode = EQuicClientVerificationMode::Pass;
	}

    // Setup client one transport
    const TSharedPtr<FQuicMessageTransportTestState> ClientTransport
        = MakeShared<FQuicMessageTransportTestState>(
            *this,                    // Test
			true,                     // bInIsClient
			ClientConfig,             // EndpointConfig
			false,                    // bInEchoEndpoint
            StaticEndpoints           // StaticEndpoints
        );
    
    // Start server and one client
    {
        AddInfo(TEXT("Starting server ..."));

        ServerTransport->Start();

        AddInfo(TEXT("Starting client ..."));

		ClientTransport->Start();
		ClientTransport->AddClient(ClientEndpoint);

		AddInfo(TEXT("Server and client should be up and running."));
    }

    /**
     * Check server and client discovery
     *
     * This test checks whether the server and the client
     * have discovered each other.
     */
    {
        AddInfo(TEXT("Checking server and client discovery ..."));

		const FTimespan MaxCheckDuration = FTimespan::FromSeconds(10);
		const FDateTime CheckStart = FDateTime::UtcNow();

		while (((ServerTransport->GetDiscoveredNodes().Num() < 1)
			|| (ClientTransport->GetDiscoveredNodes().Num() < 1))
			&& (FDateTime::UtcNow() < (CheckStart + MaxCheckDuration)))
		{
			FPlatformProcess::Sleep(0.0f);
		}

		TestTrue(
			TEXT("Server must discover exactly one node."),
			ServerTransport->GetDiscoveredNodes().Num() == 1);

        TestTrue(
            TEXT("Client must discover exactly one node."),
			ClientTransport->GetDiscoveredNodes().Num() == 1);

    	if (!ServerTransport->GetDiscoveredNodes().Contains(ClientGuid))
        {
			AddError(TEXT("Server did not discover client."));
        }

    	if (!ClientTransport->GetDiscoveredNodes().Contains(ServerGuid))
        {
			AddError(TEXT("Client did not discover server."));
        }

        if (HasAnyErrors())
        {
			return false;
        }

		AddInfo(TEXT("Client and server discovered each other."));
    }

	/**
	 * Authenticate the client with the server
	 *
	 * Since the QuicServer has endpoint authentication enabled,
	 * the QuicClient needs to send an authentication message that
	 * the server will check and if authenticated, the server will allow
	 * the client to send normal data flow messages.
	 */
    {
		AddInfo(TEXT("Sending authentication message ..."));

		ClientTransport->SendAuthMessage(ServerGuid, FQuicTestUtils::GetAuthPayload());

		AddInfo(TEXT("Sent authentication message. Waiting for response ..."));

		const FTimespan MaxCheckDuration = FTimespan::FromSeconds(10);
		const FDateTime CheckStart = FDateTime::UtcNow();

		while ((!ClientTransport->IsAuthenticated())
			&& (FDateTime::UtcNow() < (CheckStart + MaxCheckDuration)))
		{
			FPlatformProcess::Sleep(0.0f);
		}

		if (!ClientTransport->IsAuthenticated())
		{
			AddError(TEXT("Client was not authenticated."));
		}

		if (HasAnyErrors())
		{
			return false;
		}

		AddInfo(TEXT("Client is authenticated and can send normal messages now."));
    }

	/**
	 * Send small test messages from client to server
	 *
	 * This test will send a lot of small messages to the server,
	 * which will in turn send them back to the client to ensure
	 * that both directions of communication are working as intended.
	 */
    {
		const int32 MessageSize = 1024;

		AddInfo(TEXT("Sending small test messages to server ..."));

		for (int32 Count = 0; Count < NumTestMessages; ++Count)
		{
			FQuicMockMessage* MockMessage
				= FMessageEndpoint::MakeMessage<FQuicMockMessage>(MessageSize);

			const TSharedRef<FQuicMetaMessageContext> Context
				= FQuicTestUtils::WrapMessage(MockMessage);

			ClientTransport->SendMessage(ServerGuid, Context);
		}

		AddInfo(TEXT("Test messages sent to server. Waiting to receive them ..."));

		const FTimespan MaxCheckDuration = FTimespan::FromSeconds(30);
		const FDateTime CheckStart = FDateTime::UtcNow();
		
		while (((ServerTransport->GetNumReceivedMessages() < NumTestMessages)
			|| (ClientTransport->GetNumReceivedMessages() < NumTestMessages))
			&& (FDateTime::UtcNow() < (CheckStart + MaxCheckDuration)))
		{
			FPlatformProcess::Sleep(0.0f);
		}

		if (ServerTransport->GetNumReceivedMessages() != NumTestMessages)
		{
			AddError(TEXT("Server did not receive all test messages."));
		}

		// Waiting for the response from the echo server
		if (ClientTransport->GetNumReceivedMessages() != NumTestMessages)
		{
			AddError(TEXT("Client did not receive all test messages."));
		}

		if (HasAnyErrors())
		{
			return false;
		}

		AddInfo(TEXT("All endpoints received all small test messages."));
    }

	/**
	 * Send a large test message from client to server
	 *
	 * This test will send one large message to the server,
	 * which will in turn send it back to the client to ensure
	 * that both directions of communication are working as intended.
	 */
    {
		const int32 MessageSize = 100 * 1024 * 1024; // 100 MB
		const int32 ExpectedReceivedMessages = NumTestMessages + 1;

		AddInfo(TEXT("Sending large test message to the server ..."));

		FQuicMockMessage* MockMessage
			= FMessageEndpoint::MakeMessage<FQuicMockMessage>(MessageSize);

		const TSharedRef<FQuicMetaMessageContext> Context
			= FQuicTestUtils::WrapMessage(MockMessage);

		ClientTransport->SendMessage(ServerGuid, Context);

		AddInfo(TEXT("Large test message sent to the server. "
			   "Waiting to receive it on all endpoints ..."));

		const FTimespan MaxCheckDuration = FTimespan::FromSeconds(60);
		const FDateTime CheckStart = FDateTime::UtcNow();

		while (((ServerTransport->GetNumReceivedMessages() < ExpectedReceivedMessages)
			|| (ClientTransport->GetNumReceivedMessages() < ExpectedReceivedMessages))
			&& (FDateTime::UtcNow() < (CheckStart + MaxCheckDuration)))
		{
			FPlatformProcess::Sleep(0.0f);
		}

		if (ServerTransport->GetNumReceivedMessages() != ExpectedReceivedMessages)
		{
			AddError(TEXT("Server did not receive large test message."));
		}

		// Waiting for the response from the echo server
		if (ClientTransport->GetNumReceivedMessages() != ExpectedReceivedMessages)
		{
			AddError(TEXT("Client did not receive large test message."));
		}

		if (HasAnyErrors())
		{
			return false;
		}

		AddInfo(TEXT("All endpoints received the large test message."));
    }

	// Shutdown original client to start a new, unauthenticated one
    {
		ClientTransport->RemoveClient(ClientEndpoint);
		ClientTransport->Stop();

		FPlatformProcess::Sleep(3.0f);
    }

	// Create another client Guid and adapt the client config
	const FGuid AnotherClientGuid = FGuid::NewGuid();
	ClientConfig->LocalNodeId = AnotherClientGuid;

	// Setup another client transport
	const TSharedPtr<FQuicMessageTransportTestState> ClientTransportNoAuth
		= MakeShared<FQuicMessageTransportTestState>(
			*this,                    // Test
			true,                     // bInIsClient
			ClientConfig,             // EndpointConfig
			false,                    // bInEchoEndpoint
			StaticEndpoints           // StaticEndpoints
		);

	/**
	 * Connect and send message with unauthenticated client
	 *
	 * This test checks whether an unauthenticated client can send normal
	 * data messages to the server. That should not work and would throw an error
	 * on the server side, which is caught as to not make the test fail.
	 */
    {
		AddInfo(TEXT("Connecting with another client ..."));

		ClientTransportNoAuth->Start();
		ClientTransportNoAuth->AddClient(ClientEndpoint);

		FPlatformProcess::Sleep(3.0f);

		if (!ServerTransport->GetDiscoveredNodes().Contains(AnotherClientGuid))
		{
			AddError(TEXT("Server did not discover new client."));
		}

		const int32 MessageSize = 2048;
		const int32 ExpectedReceivedMessages = NumTestMessages + 2;

		AddExpectedError(TEXT("Unauthenticated handler sending data"));

		AddInfo(TEXT("Sending unauthenticated test message to the server ..."));

		FQuicMockMessage* MockMessage
			= FMessageEndpoint::MakeMessage<FQuicMockMessage>(MessageSize);

		const TSharedRef<FQuicMetaMessageContext> Context
			= FQuicTestUtils::WrapMessage(MockMessage);

		ClientTransportNoAuth->SendMessage(ServerGuid, Context);

		AddInfo(TEXT("Unauthenticated test message sent to the server."));

		FPlatformProcess::Sleep(3.0f);

		TestFalse(
			TEXT("Server shouldn't receive unauthenticated message."),
			ServerTransport->GetNumReceivedMessages() == ExpectedReceivedMessages);

		if (HasAnyErrors())
		{
			return false;
		}

		AddInfo(TEXT("Server did not receive unauthenticated message."));
    }

	return true;
}


void EmptyLinkFunctionForStaticInitializationQuicMessageTransportTest()
{
	// This function exists to prevent the object file containing this test
	// from being excluded by the linker, because it has no publicly referenced symbols.
}

