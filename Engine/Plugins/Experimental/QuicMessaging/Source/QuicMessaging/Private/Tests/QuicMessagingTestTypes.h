// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IMessageContext.h"
#include "QuicMessagingTestTypes.generated.h"


USTRUCT()
struct FQuicMockMessage
{

	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	TArray<uint8> Data;

	FQuicMockMessage()
	{
		Data.AddUninitialized(64);
	}

	FQuicMockMessage(int32 DataSize)
	{
		Data.AddUninitialized(DataSize);
	}

};
