// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/QuicMessageProcessor.h"
#include "QuicMessagingPrivate.h"

#include "HAL/Event.h"
#include "HAL/RunnableThread.h"
#include "IMessageAttachment.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Math/UnrealMathUtility.h"
#include "UObject/Class.h"

#include "Backends/CborStructSerializerBackend.h"
#include "StructSerializer.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "StructDeserializer.h"

#include "IMessageContext.h"
#include "INetworkMessagingExtension.h"

#include "QuicSerializedMessage.h"
#include "QuicDeserializedMessage.h"
#include "QuicMessagingSettings.h"

#include "QuicEndpointManager.h"


namespace UE::Private::MessageProcessor
{

	FOnOutboundTransferDataUpdated& OnOutboundUpdated()
	{
		static FOnOutboundTransferDataUpdated OnTransferUpdated;
		return OnTransferUpdated;
	}

	FOnInboundTransferDataUpdated& OnInboundUpdated()
	{
		static FOnInboundTransferDataUpdated OnTransferUpdated;
		return OnTransferUpdated;
	}

}


FQuicMessageProcessor::FQuicMessageProcessor()
    : bStopping(false)
    , bIsInitialized(false)
    , MessageFormat(GetDefault<UQuicMessagingSettings>()->MessageFormat)
	, Thread(nullptr)
	, SerializeWorkEvent(nullptr)
	, DeserializeWorkEvent(nullptr)
	, bUpdatingSerialization(false)
	, PendingSerializeMessages(TMap<uint32, FQuicSerializedMessagePtr>())
	, PendingDeserializeMessages(TMap<uint32, FQuicDeserializedMessagePtr>())
	, SerializeMessagesQueue(TQueue<FQuicSerializedMessagePtr>())
	, DeserializeMessagesQueue(TQueue<FQuicDeserializedMessagePtr>())
{
	SerializeWorkEvent = MakeShareable(FPlatformProcess::GetSynchEventFromPool(), [](FEvent* EventToDelete)
    {
        FPlatformProcess::ReturnSynchEventToPool(EventToDelete);
    });

	DeserializeWorkEvent = MakeShareable(FPlatformProcess::GetSynchEventFromPool(), [](FEvent* EventToDelete)
    {
        FPlatformProcess::ReturnSynchEventToPool(EventToDelete);
    });

    Thread = FRunnableThread::Create(this, *FString::Printf(TEXT("FQuicMessageProcessor-%d"), FMath::RandRange(0, 10)), 128 * 4096,
        TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());

	UE_LOGF(LogQuicMessaging, Verbose, "[MessageProcessor] Started message processor.");
}


FQuicMessageProcessor::~FQuicMessageProcessor()
{
    Thread->Kill(true);
    delete Thread;
    Thread = nullptr;
}


void FQuicMessageProcessor::ProcessInboundMessage(const FInboundMessage InboundMessage)
{
    if (bStopping)
    {
        return;
    }

	TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage
		= MakeShared<FQuicDeserializedMessage, ESPMode::ThreadSafe>(InboundMessage);

	DeserializeMessagesQueue.Enqueue(DeserializedMessage);
}


bool FQuicMessageProcessor::ProcessOutboundMessage(
    const TSharedRef<IMessageContext, ESPMode::ThreadSafe> MessageContext,
    const TArray<FGuid>& Recipients, const EQuicMessageType MessageType)
{
    if (bStopping)
    {
        return false;
    }

	FQuicPayloadPtr DataPtr = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();

    TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe> SerializedMessage
        = MakeShared<FQuicSerializedMessage, ESPMode::ThreadSafe>(
		MessageFormat, MessageContext->GetFlags(), Recipients, MessageType, DataPtr, MessageContext);

	SerializeMessagesQueue.Enqueue(SerializedMessage);

	return true;
}


void FQuicMessageProcessor::ConsumeMessages()
{
	if (bStopping || bConsumingMessages)
	{
		return;
	}

	bConsumingMessages = true;

	const double CurrentTime = FPlatformTime::Seconds();

	if (CurrentTime < (LastConsumed + ConsumeInterval.GetTotalSeconds()))
	{
		bConsumingMessages = false;
		return;
	}

	if (SerializeMessagesQueue.IsEmpty() && DeserializeMessagesQueue.IsEmpty())
	{
		bConsumingMessages = false;
		return;
	}

	TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe> SerializedMessage;

	while (!bStopping && SerializeMessagesQueue.Dequeue(SerializedMessage))
	{
		uint32 SerializeKey = SerializeTaskId;
		SerializeTaskId++;

		PendingSerializeMessages.Add(SerializeKey, SerializedMessage);

		TGraphTask<FQuicSerializeMessageTask>::CreateTask()
			.ConstructAndDispatchWhenReady(
				SerializedMessage->GetMessageContext(),
				SerializedMessage, SerializeWorkEvent);
	}

	TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage;

	while (!bStopping && DeserializeMessagesQueue.Dequeue(DeserializedMessage))
	{
		uint32 DeserializeKey = DeserializeTaskId;
		DeserializeTaskId++;

		PendingDeserializeMessages.Add(DeserializeKey, DeserializedMessage);

		TGraphTask<FQuicDeserializeMessageTask>::CreateTask()
			.ConstructAndDispatchWhenReady(
				DeserializedMessage->GetInboundMessage().UnserializedMessage,
				DeserializedMessage, DeserializeWorkEvent);
	}

	LastConsumed = FPlatformTime::Seconds();

	bConsumingMessages = false;
}


void FQuicMessageProcessor::UpdateSerialization()
{
	if (bStopping || bUpdatingSerialization)
	{
		return;
	}

	bUpdatingSerialization = true;

	const double CurrentTime = FPlatformTime::Seconds();

	if (CurrentTime < (LastSerialization + SerializationInterval.GetTotalSeconds()))
	{
		bUpdatingSerialization = false;
		return;
	}

	for (auto SerializeIt = PendingSerializeMessages.CreateIterator(); SerializeIt; ++SerializeIt)
	{
		TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe> SerializedMessage = SerializeIt->Value;

		if (!SerializedMessage.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error, "[MessageProcessor] Serialized message ptr invalid.");

			SerializeIt.RemoveCurrent();

			continue;
		}

		if (SerializedMessage->GetState() == EQuicSerializedMessageState::Complete)
		{
			MessageSerializedDelegate.ExecuteIfBound(SerializedMessage);

			FOnOutboundTransferDataUpdated& OutboundUpdated = UE::Private::MessageProcessor::OnOutboundUpdated();

			SerializeIt.RemoveCurrent();
		}

		if (SerializedMessage->GetState() == EQuicSerializedMessageState::Invalid)
		{
			UE_LOGF(LogQuicMessaging, Error, "[MessageProcessor] Message serialize failed.");

			SerializeIt.RemoveCurrent();
		}
	}

	for (auto DeserializeIt = PendingDeserializeMessages.CreateIterator(); DeserializeIt; ++DeserializeIt)
	{
		TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage = DeserializeIt->Value;

		if (!DeserializedMessage.IsValid())
		{
			UE_LOGF(LogQuicMessaging, Error, "[MessageProcessor] Deserialized message pointer invalid.");

			DeserializeIt.RemoveCurrent();

			continue;
		}

		if (DeserializedMessage->GetState() == EQuicDeserializedMessageState::Complete)
		{
			MessageDeserializedDelegate.ExecuteIfBound(DeserializedMessage,
				DeserializedMessage->GetInboundMessage().MessageHeader.SenderId);

			FOnInboundTransferDataUpdated& InboundUpdated = UE::Private::MessageProcessor::OnInboundUpdated();

			DeserializeIt.RemoveCurrent();
		}

		if (DeserializedMessage->GetState() == EQuicDeserializedMessageState::Invalid)
		{
			DeserializeIt.RemoveCurrent();
		}
	}

	LastSerialization = FPlatformTime::Seconds();

	bUpdatingSerialization = false;

}


FQuicPayloadPtr FQuicMessageProcessor::SerializeMessageHeader(FMessageHeader& InMessageHeader) const
{
	FQuicPayloadPtr SerializedData = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();

	SerializedData->Reserve(QUIC_MESSAGE_HEADER_SIZE);

	FMemoryWriter Archive(*SerializedData.Get(), true);

	uint8 MessageType = static_cast<uint8>(InMessageHeader.MessageType);
	Archive << MessageType;

	Archive << InMessageHeader.RecipientId;

	Archive << InMessageHeader.SenderId;
	
	Archive << InMessageHeader.SerializedMessageSize;

	SerializedData->AddDefaulted(QUIC_MESSAGE_HEADER_SIZE - Archive.TotalSize());

	return SerializedData;
}


FMessageHeader FQuicMessageProcessor::DeserializeMessageHeader(FQuicPayloadPtr InHeaderData) const
{
	FMessageHeader MessageHeader;

	FMemoryReader Archive(*InHeaderData);

	uint8 MessageType;
	Archive << MessageType;
	MessageHeader.MessageType = static_cast<EQuicMessageType>(MessageType);

	FGuid RecipientId;
	Archive << RecipientId;
	MessageHeader.RecipientId = RecipientId;

	FGuid SenderId;
	Archive << SenderId;
	MessageHeader.SenderId = SenderId;

	uint32 SerializedMessageSize;
	Archive << SerializedMessageSize;
	MessageHeader.SerializedMessageSize = SerializedMessageSize;

	return MessageHeader;
}


void FQuicMessageProcessor::WaitAsyncTaskCompletion()
{
    // Stop the processor thread
    Stop();
}


FSingleThreadRunnable* FQuicMessageProcessor::GetSingleThreadInterface()
{
	return this;
}


uint32 FQuicMessageProcessor::Run()
{
    while (!bStopping)
    {
		ConsumeMessages();
		UpdateSerialization();
    }

    return 0;
}


void FQuicMessageProcessor::Stop()
{
	bStopping = true;
	SerializeWorkEvent->Trigger();
	DeserializeWorkEvent->Trigger();
}




