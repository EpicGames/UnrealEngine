// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QuicMessagingPrivate.h"
#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "HAL/Runnable.h"
#include "IMessageTransport.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/SingleThreadRunnable.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"
#include "INetworkMessagingExtension.h"

#include "QuicSerializationTasks.h"

#include "QuicFlags.h"
#include "QuicMessages.h"


class FArrayReader;
class FEvent;
class FRunnableThread;

class FQuicEndpointManager;
struct FInboundMessage;

class FQuicSerializedMessage;
class FQuicDeserializedMessage;
class IMessageAttachment;
class IMessageContext;
enum class EQuicMessageFormat : uint8;


/**
 * @note: The delegate macro recognizes the ThreadSafe TSharedPtr as two arguments.
 */
using FQuicDeserializedMessagePtr = TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe>;
using FQuicSerializedMessagePtr = TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe>;


namespace UE::Private::MessageProcessor
{

/**
 * Global delegate for handling outbound processor updates.
 */
FOnOutboundTransferDataUpdated& OnOutboundUpdated();

/**
 * Global delegate for handling inbound processor updates.
 */
FOnInboundTransferDataUpdated& OnInboundUpdated();
}


/**
 * Implements a message processor for QUIC messages.
 */
class FQuicMessageProcessor
	: public FRunnable
	, private FSingleThreadRunnable
{

public:

	/**
	 * Creates and initializes a new message processor.
	 */
	FQuicMessageProcessor();

	/** Virtual destructor. */
	virtual ~FQuicMessageProcessor();

public:

	/**
	 * Prepares and enqueues an inbound message for deserialization.
	 *
	 * @param InboundMessage The inbound message
	 */
	void ProcessInboundMessage(const FInboundMessage InboundMessage);

	/**
	 * Prepares and enqueues an outbound message for serialization.
	 *
	 * @param MessageContext The message to serialize
	 * @param Recipients The recipient nodes ids
	 * @param MessageType The message type
	 * 
	 * @return Whether the message could be enqueued
	 */
	bool ProcessOutboundMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe> MessageContext,
		const TArray<FGuid>& Recipients, const EQuicMessageType MessageType);

private:

	/**
	 * Consume messages and create task to serialize and deserialize messages.
	 */
	void ConsumeMessages();

public:

	/**
	 * Updates message serialization tasks.
	 */
	void UpdateSerialization();

	/**
	 * Serializes a message header.
	 *
	 * @param InMessageHeader The message header to serialize
	 * @return The serialized message header
	 */
	FQuicPayloadPtr SerializeMessageHeader(FMessageHeader& InMessageHeader) const;

	/**
	 * Deserializes a message header.
	 *
	 * @param InHeaderData The message header data to deserialize
	 * @return The deserialized message header
	 */
	FMessageHeader DeserializeMessageHeader(FQuicPayloadPtr InHeaderData) const;

public:

	/**
	 * Waits for all serialization tasks fired by this processor to complete.
	 * Expected to be called when the application exit to prevent
	 * serialized (UStruct) object to being use after the UObject system is shutdown.
	 */
	void WaitAsyncTaskCompletion();

public:

	/**
	 * Returns a delegate that is executed when a message has been deserialized.
	 *
	 * @return The delegate.
	 */
	DECLARE_DELEGATE_TwoParams(FOnMessageDeserialized, FQuicDeserializedMessagePtr /*DeserializedMessage*/, FGuid /*NodeId*/)
	FOnMessageDeserialized& OnMessageDeserialized()
	{
		return MessageDeserializedDelegate;
	}

	/**
	 * Returns a delegate that is executed when a message has been serialized.
	 *
	 * @return The delegate.
	 */
	DECLARE_DELEGATE_OneParam(FOnMessageSerialized, FQuicSerializedMessagePtr /*SerializedMessage*/)
	FOnMessageSerialized& OnMessageSerialized()
	{
		return MessageSerializedDelegate;
	}

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override;
	virtual bool Init() override { return true; };
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override { }

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override { };

private:

	/** Holds a flag indicating that the thread is stopping. */
	bool bStopping;

	/** Holds a flag indicating if the processor is initialized. */
	bool bIsInitialized;

	/** The configured message format (from UQuicMessagingSettings). */
	EQuicMessageFormat MessageFormat;

	/** Holds the thread object. */
	FRunnableThread* Thread;

	/** Holds an event signaling that outbound messages need to be processed. */
	TSharedPtr<FEvent, ESPMode::ThreadSafe> SerializeWorkEvent;

	/** Holds an event signaling that inbound messages need to be processed. */
	TSharedPtr<FEvent, ESPMode::ThreadSafe> DeserializeWorkEvent;

private:

	/** Holds a flag indicating if the processor is updating serialization. */
	bool bUpdatingSerialization;

	/** Holds the interval for message de/serialization. */
	const FTimespan SerializationInterval = 10000;

	/** Holds timestamp of last de/serialization. */
	double LastSerialization = FPlatformTime::Seconds();

	/** Holds the current serialize task id. */
	uint32 SerializeTaskId = 0;

	/** Holds the current deserialize task id. */
	uint32 DeserializeTaskId = 0;

	/** Holds pending serialized messages. */
	TMap<uint32, FQuicSerializedMessagePtr> PendingSerializeMessages;

	/** Holds pending deserialized messages. */
	TMap<uint32, FQuicDeserializedMessagePtr> PendingDeserializeMessages;

private:

	/** Flag indicating if messages are currently being consumed.*/
	bool bConsumingMessages = false;

	/** Holds the interval for message consume. */
	const FTimespan ConsumeInterval = 10000;

	/** Holds the timestamp of when messages have last been consumed. */
	double LastConsumed = FPlatformTime::Seconds();

	/** Queue that holds serialized messages. */
	TQueue<FQuicSerializedMessagePtr> SerializeMessagesQueue;

	/** Queue that holds deserialized messages. */
	TQueue<FQuicDeserializedMessagePtr> DeserializeMessagesQueue;

private:

	/** Holds a delegate to be invoked when a message has been deserialized. */
	FOnMessageDeserialized MessageDeserializedDelegate;

	/** Holds a delegate to be invoked when a message has been serialized. */
	FOnMessageSerialized MessageSerializedDelegate;
	
};
