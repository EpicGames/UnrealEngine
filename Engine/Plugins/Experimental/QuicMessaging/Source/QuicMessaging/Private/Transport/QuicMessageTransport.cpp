// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/QuicMessageTransport.h"

#include "HAL/RunnableThread.h"
#include "IMessageContext.h"
#include "IMessageTransportHandler.h"
#include "Misc/Guid.h"
#include "Serialization/ArrayReader.h"
#include "Async/Async.h"

#include "QuicMessagingPrivate.h"
#include "QuicMessages.h"
#include "QuicEndpointConfig.h"
#include "Transport/QuicMessageProcessor.h"

#include "QuicEndpointManager.h"


FQuicMessageTransport::FQuicMessageTransport(const bool bInIsClient,
	const TSharedRef<FQuicEndpointConfig> InEndpointConfig,
	TArray<FIPv4Endpoint> InStaticEndpoints)
	: EndpointConfig(InEndpointConfig)
    , StaticEndpoints(MoveTemp(InStaticEndpoints))
{
    EndpointMode = (bInIsClient) ? EEndpointMode::Client : EEndpointMode::Server;
}


FQuicMessageTransport::~FQuicMessageTransport()
{
	StopTransport();
}


void FQuicMessageTransport::OnAppPreExit()
{
	// Remove any bound delegates. It's no longer relevant for us to
	// send a transport error when we are in the shutdown phase.
	TransportErrorDelegate.Unbind();

	if (MessageProcessor.IsValid())
	{
		MessageProcessor->WaitAsyncTaskCompletion();
	}
}


void FQuicMessageTransport::AddStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	if (EndpointManager == nullptr)
	{
		return;
	}

	if (EndpointMode == EEndpointMode::Server)
	{
		return;
	}

    EndpointManager->AddClient(InEndpoint);
}


void FQuicMessageTransport::RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	if (EndpointManager != nullptr)
	{
		EndpointManager->RemoveClient(InEndpoint);
	}
}


TArray<FIPv4Endpoint> FQuicMessageTransport::GetListeningAddresses() const
{
	TArray<FIPv4Endpoint> Listeners;

	Listeners.Add(EndpointConfig->Endpoint);

	return Listeners;
}


TArray<FIPv4Endpoint> FQuicMessageTransport::GetKnownEndpoints() const
{
	if (EndpointManager)
	{
		return EndpointManager->GetKnownEndpoints();
	}

	return {};
}


FMessageTransportStatistics FQuicMessageTransport::GetLatestStatistics(FGuid NodeId) const
{
	if (EndpointManager)
	{
		return EndpointManager->GetStats(NodeId);
	}

	return {};
}


void FQuicMessageTransport::SetMaxAuthenticationMessageSize(const uint32 MaxBytes) const
{
	if (!EndpointManager)
	{
		return;
	}

	if (EndpointMode != EEndpointMode::Server)
	{
		return;
	}

	const TSharedRef<FQuicServerConfig> ServerEndpointConfig
		= StaticCastSharedRef<FQuicServerConfig, FQuicEndpointConfig>(EndpointConfig);

	ServerEndpointConfig->MaxAuthenticationMessageSize = MaxBytes;
}


TOptional<FGuid> FQuicMessageTransport::GetNodeId(const FIPv4Endpoint& RemoteEndpoint) const
{
	if (!EndpointManager)
	{
		return TOptional<FGuid>();
	}

	return EndpointManager->GetKnownNodeId(RemoteEndpoint);
}


bool FQuicMessageTransport::IsNodeAuthenticated(const FGuid& NodeId) const
{
    if (EndpointManager == nullptr)
    {
        return false;
    }

    return EndpointManager->IsEndpointAuthenticated(NodeId);
}


void FQuicMessageTransport::SetNodeAuthenticated(const FGuid& NodeId)
{
    if (EndpointManager == nullptr)
    {
        return;
    }

	EndpointManager->SetEndpointAuthenticated(NodeId);
}


void FQuicMessageTransport::DisconnectNode(const FGuid& NodeId) const
{
	if (!EndpointManager)
	{
		return;
	}

	EndpointManager->DisconnectNode(NodeId);
}


void FQuicMessageTransport::SetConnectionCooldown(
	const bool bEnabled, const uint32 MaxAttempts, const uint32 PeriodSeconds,
	const uint32 CooldownSeconds, const uint32 CooldownMaxSeconds) const
{
	if (!EndpointManager)
	{
		return;
	}

	if (EndpointMode != EEndpointMode::Server)
	{
		return;
	}

	const TSharedRef<FQuicServerConfig> ServerEndpointConfig
		= StaticCastSharedRef<FQuicServerConfig, FQuicEndpointConfig>(EndpointConfig);

	ServerEndpointConfig->ConnCooldownMode = (bEnabled)
		? EConnectionCooldownMode::Enabled : EConnectionCooldownMode::Disabled;

	ServerEndpointConfig->ConnCooldownMaxAttempts = MaxAttempts;
	ServerEndpointConfig->ConnCooldownPeriodSec = PeriodSeconds;
	ServerEndpointConfig->ConnCooldownSec = CooldownSeconds;
	ServerEndpointConfig->ConnCooldownMaxSec = CooldownMaxSeconds;
}


bool FQuicMessageTransport::RestartTransport()
{
	UE_LOGF(LogQuicMessaging, Verbose, "[MessageTransport] Restarting transport ...");

	IMessageTransportHandler* Handler = TransportHandler;
	StopTransport();
	return StartTransport(*Handler);
}


/**
 * IMessageTransport interface.
 */

FName FQuicMessageTransport::GetDebugName() const
{
	return "QuicMessageTransport";
}


bool FQuicMessageTransport::StartTransport(IMessageTransportHandler& Handler)
{
    TransportHandler = &Handler;

	EndpointManager = MakeUnique<FQuicEndpointManager>(EndpointConfig);

	EndpointManager->OnMessageDelivered().BindRaw(this, &FQuicMessageTransport::HandleManagerMessageDelivered);
	EndpointManager->OnMessageValidated().BindRaw(this, &FQuicMessageTransport::HandleManagerMessageReceived);
	EndpointManager->OnSerializeHeader().BindRaw(this, &FQuicMessageTransport::HandleManagerSerializeHeader);
	EndpointManager->OnDeserializeHeader().BindRaw(this, &FQuicMessageTransport::HandleManagerDeserializeHeader);
	EndpointManager->OnEndpointNodeDiscovered().BindRaw(this, &FQuicMessageTransport::HandleManagerNodeDiscovered);
	EndpointManager->OnEndpointNodeLost().BindRaw(this, &FQuicMessageTransport::HandleManagerNodeLost);
	EndpointManager->OnClientConnectionChanged().BindRaw(this, &FQuicMessageTransport::HandleClientConnectionChange);

	if (EndpointMode == EEndpointMode::Server)
	{
		EndpointManager->InitializeServer();
	}

	MessageProcessor = MakeUnique<FQuicMessageProcessor>();

    MessageProcessor->OnMessageDeserialized().BindRaw(this, &FQuicMessageTransport::HandleProcessorMessageDeserialized);
    MessageProcessor->OnMessageSerialized().BindRaw(this, &FQuicMessageTransport::HandleProcessorMessageSerialized);
	
    TSet<FIPv4Endpoint> ProcessStaticEndpints(StaticEndpoints);

    for (const FIPv4Endpoint& StaticEndpoint : ProcessStaticEndpints)
    {
		if (StaticEndpoint == FIPv4Endpoint())
		{
			continue;
		}

        AddStaticEndpoint(StaticEndpoint);
    }

    UE_LOGF(LogQuicMessaging, Verbose, "[MessageTransport] Started transport.");

    return true;
}


void FQuicMessageTransport::StopTransport()
{
	UE_LOGF(LogQuicMessaging, Verbose, "[MessageTransport] Stopping transport ...");

	if (EndpointManager.IsValid())
	{
		EndpointManager.Reset();
		EndpointManager = nullptr;
	}

    if (MessageProcessor.IsValid())
    {
		MessageProcessor.Reset();
        MessageProcessor = nullptr;
    }

    TransportHandler = nullptr;

	if (ErrorFuture.IsValid())
	{
		ErrorFuture.Reset();
	}

    UE_LOGF(LogQuicMessaging, Verbose, "[MessageTransport] Stopped transport.");
}


bool FQuicMessageTransport::TransportMessage(
    const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    const TArray<FGuid>& Recipients)
{
    if (MessageProcessor == nullptr)
    {
        return false;
    }

    if (Context->GetRecipients().Num() > QUIC_MESSAGING_MAX_RECIPIENTS)
    {
        return false;
    }

    return MessageProcessor->ProcessOutboundMessage(Context, Recipients, EQuicMessageType::Data);
}


bool FQuicMessageTransport::TransportAuthMessage(
    const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    const FGuid& Recipient)
{
    if (MessageProcessor == nullptr)
    {
        return false;
    }

	const TArray<FGuid> Recipients = { Recipient };

    return MessageProcessor->ProcessOutboundMessage(Context, Recipients, EQuicMessageType::Authentication);
}


bool FQuicMessageTransport::TransportAuthResponseMessage(
    const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    const FGuid& Recipient)
{
    if (MessageProcessor == nullptr)
    {
        return false;
    }

	const TArray<FGuid> Recipients = { Recipient };

    return MessageProcessor->ProcessOutboundMessage(Context, Recipients, EQuicMessageType::AuthenticationResponse);
}


FOnQuicMetaMessageReceived& FQuicMessageTransport::OnMetaMessageReceived()
{
	return OnQuicMetaMessageDelegate;
}


FOnQuicClientConnectionChanged& FQuicMessageTransport::OnClientConnectionChanged()
{
	return OnQuicClientConnectionChangedDelegate;
}


void FQuicMessageTransport::HandleProcessorMessageDeserialized(
	const TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage,
	const FGuid NodeId) const
{
	if (!DeserializedMessage.IsValid())
	{
		UE_LOGF(LogQuicMessaging, Error,
			"[MessageTransport] Deserialized message pointer invalid.");

		return;
	}

	// Receive authentication messages separately
	if (DeserializedMessage->GetMessageTypeInfo()
		->IsChildOf(FQuicMetaMessage::StaticStruct()))
	{
		OnQuicMetaMessageDelegate.Broadcast(NodeId, DeserializedMessage.ToSharedRef());

		return;
	}

	TransportHandler->ReceiveTransportMessage(DeserializedMessage.ToSharedRef(), NodeId);
}


void FQuicMessageTransport::HandleProcessorMessageSerialized(
	TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe> SerializedMessage)
{
	if (EndpointManager == nullptr)
	{
		return;
	}

	const bool bIsBroadcast = SerializedMessage->GetRecipients().IsEmpty();

	TArray<FGuid> Recipients = bIsBroadcast
		? EndpointManager->GetKnownNodeIds() : SerializedMessage->GetRecipients();

	TArray<TTuple<FGuid, uint32>> MessageMetas;

	for (const FGuid& Recipient : Recipients)
	{
		const uint32 MessageId = SerializedMessageId;

		SerializedMessages.Add(MessageId, SerializedMessage);
		MessageMetas.Add(TTuple<FGuid, uint32>(Recipient, MessageId));

		SerializedMessageId++;
	}

	EndpointManager->EnqueueOutboundMessages(
		SerializedMessage->GetDataPtr(), MessageMetas,
		SerializedMessage->GetMessageType());
}


void FQuicMessageTransport::HandleManagerMessageDelivered(const uint32 MessageId)
{
	if (!SerializedMessages.Contains(MessageId))
	{
		return;
	}

	SerializedMessages.Remove(MessageId);
}


void FQuicMessageTransport::HandleManagerMessageReceived(const FInboundMessage InboundMessage)
{
	if (MessageProcessor == nullptr)
	{
		return;
	}

	MessageProcessor->ProcessInboundMessage(InboundMessage);
}


FQuicPayloadPtr FQuicMessageTransport::HandleManagerSerializeHeader(FMessageHeader& MessageHeader)
{
	if (MessageProcessor == nullptr)
	{
		return MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();
	}

	return MessageProcessor->SerializeMessageHeader(MessageHeader);
}


FMessageHeader FQuicMessageTransport::HandleManagerDeserializeHeader(FQuicPayloadPtr HeaderData)
{
	if (MessageProcessor == nullptr)
	{
		return FMessageHeader();
	}

	return MessageProcessor->DeserializeMessageHeader(HeaderData);
}


void FQuicMessageTransport::HandleManagerNodeDiscovered(const FGuid& DiscoveredNodeId)
{
	UE_LOGF(LogQuicMessaging, Display,
		"[MessageProcessor] Discovered node %ls.", *DiscoveredNodeId.ToString());

	TransportHandler->DiscoverTransportNode(DiscoveredNodeId);
}


void FQuicMessageTransport::HandleManagerNodeLost(const FGuid& LostNodeId)
{
	UE_LOGF(LogQuicMessaging, Display,
		"[MessageProcessor] Lost node %ls.", *LostNodeId.ToString());

	TransportHandler->ForgetTransportNode(LostNodeId);
}


void FQuicMessageTransport::HandleClientConnectionChange(
	const FGuid& NodeId, const FIPv4Endpoint& RemoteEndpoint,
	const EQuicClientConnectionChange ConnectionState)
{
	EQuicClientConnectionState ClientState
		= (ConnectionState == EQuicClientConnectionChange::Connected)
		? EQuicClientConnectionState::Connected : EQuicClientConnectionState::Disconnected;

	OnQuicClientConnectionChangedDelegate.Broadcast(NodeId, RemoteEndpoint, ClientState);
}
