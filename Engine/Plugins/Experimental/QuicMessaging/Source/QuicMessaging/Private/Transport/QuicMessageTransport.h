// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMessageTransport.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "IMessageAttachment.h"
#include "Templates/SharedPointer.h"
#include "Async/Future.h"
#include "Containers/Ticker.h"
#include "INetworkMessagingExtension.h"

#include "Serialization/QuicSerializedMessage.h"
#include "Serialization/QuicDeserializedMessage.h"

#include "QuicFlags.h"
#include "QuicMessages.h"
#include "QuicTransportMessages.h"
#include "QuicTransportNotifications.h"


class FArrayReader;
class FRunnableThread;

class FQuicEndpointManager;
struct FInboundMessage;
struct FQuicEndpointConfig;

class FQuicMessageProcessor;
class IMessageAttachment;
class IMessageContext;
class IMessageTransportHandler;
enum class EQuicMessageFormat : uint8;

struct FGuid;


/**
* Implements a message transport technology using mvfst with QUIC.
*/
class FQuicMessageTransport
	: public TSharedFromThis<FQuicMessageTransport, ESPMode::ThreadSafe>
	, public IMessageTransport
{
public:

	/** Default constructor. */
	FQuicMessageTransport() = delete;

	/**
	 * Creates and initializes a new transport instance.
	 *
	 * @param bInIsClient Whether the transport is in client mode
	 * @param InEndpointConfig The endpoint config
	 * @param InStaticEndpoints The static endpoints
	 */
	FQuicMessageTransport(const bool bInIsClient,
		const TSharedRef<FQuicEndpointConfig> InEndpointConfig,
		TArray<FIPv4Endpoint> InStaticEndpoints);

	/** Virtual destructor. */
	virtual ~FQuicMessageTransport();

	/** Notifies the transport that the application is about to exit. */
	void OnAppPreExit();

public:

	/**
	 * Add a static server endpoint to the transport.
	 * 
	 * @param InEndpoint The static endpoint to add
	 */
	void AddStaticEndpoint(const FIPv4Endpoint& InEndpoint);

	/**
	 * Remove a static server endpoint from the transport.
	 * 
	 * @param InEndpoint The static endpoint to remove
	 */
	void RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint);

	/**
	 * Get the IPv4Endpoint listening addresses.
	 */
	TArray<FIPv4Endpoint> GetListeningAddresses() const;

	/**
	 * Get the known endpoints that are in active communication with the QUIC transport.
	 * This is information known by the EndpointManager.
	 */
	TArray<FIPv4Endpoint> GetKnownEndpoints() const;

	/**
	 * Get the latest network/transport statistics for a node.
	 *
	 * @param NodeId The node ID
	 */
	FMessageTransportStatistics GetLatestStatistics(FGuid NodeId) const;

public:

	/**
	 * Set the maximum authentication message size.
	 *
	 * @param MaxBytes The maximum message size in bytes
	 */
	void SetMaxAuthenticationMessageSize(const uint32 MaxBytes) const;

	/**
	 * Get the NodeId of the remote endpoint.
	 *
	 * @param RemoteEndpoint The remote endpoint address
	 *
	 * @return Optional FGuid if the node was found
	 */
	TOptional<FGuid> GetNodeId(const FIPv4Endpoint& RemoteEndpoint) const;

	/**
	 * Check if a node is authenticated.
	 *
	 * @param NodeId The node identifier
	 */
	bool IsNodeAuthenticated(const FGuid& NodeId) const;

	/**
	 * Mark a node as authenticated.
	 *
	 * @param NodeId The node identifier
	 */
	void SetNodeAuthenticated(const FGuid& NodeId);

	/**
	 * Disconnect a node.
	 *
	 * @param NodeId The node identifier
	 */
	void DisconnectNode(const FGuid& NodeId) const;

	/**
	 * Set the connection cooldown.
	 *
	 * @param bEnabled Whether the cooldown should be enabled
	 * @param MaxAttempts Maximum number of attempts
	 * @param PeriodSeconds Time period for maximum number of attempts in seconds
	 * @param CooldownSeconds Duration of the cooldown in seconds
	 * @param CooldownMaxSeconds Max duration of the cooldown in seconds
	 */
	void SetConnectionCooldown(
		const bool bEnabled, const uint32 MaxAttempts, const uint32 PeriodSeconds,
		const uint32 CooldownSeconds, const uint32 CooldownMaxSeconds) const;

public:

	/**
	 * Returns a delegate that is exectued when a connection error happened.
	 * 
	 * @return The delegate
	 * @note this delegate is executed from the main thread by the task graph
	 */
	DECLARE_DELEGATE(FOnError)
	FOnError& OnTransportError()
	{
		return TransportErrorDelegate;
	}

	/**
	 * Restart the transport using the same Transport handler
	 * @return true if the restart was successful
	 */
	bool RestartTransport();

	//~ IMessageTransport interface

	virtual FName GetDebugName() const override;

	virtual bool StartTransport(IMessageTransportHandler& Handler) override;

	virtual void StopTransport() override;

	virtual bool TransportMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
		const TArray<FGuid>& Recipients) override;

public:

	/**
	 * Transport an authentication message.
	 *
	 * @param Context The message context
	 * @param Recipient The message recipient
	 */
	bool TransportAuthMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    	const FGuid& Recipient);

	/**
	 * Transport an authentication response message.
	 *
	 * @param Context The message context
	 * @param Recipient The message recipient
	 */
	bool TransportAuthResponseMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    	const FGuid& Recipient);

	/**
	 * Delegate that notifies bound functions when
	 * a meta / authentication message has been received.
	 *
	 * @return The delegate
	 */
	FOnQuicMetaMessageReceived& OnMetaMessageReceived();

	/**
	 * Delegate that notifies bound functions when a client
	 * has connected to or disconnected from a remote endpoint (QuicServer).
	 */
	FOnQuicClientConnectionChanged& OnClientConnectionChanged();

private:

	/**
	 * Handles deserialized transport messages.
	 */
	void HandleProcessorMessageDeserialized(
		const TSharedPtr<FQuicDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage,
		const FGuid NodeId) const;

	/**
	 * Handles serialized transport messages.
	 */
	void HandleProcessorMessageSerialized(
		TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe> SerializedMessage);

private:

	/**
	 * Handles delivered messages.
	 */
	void HandleManagerMessageDelivered(const uint32 MessageId);

	/**
	 * Handles received transport messages.
	 */
	void HandleManagerMessageReceived(const FInboundMessage InboundMessage);

	/**
	 * Handles message headers that need to be serialized.
	 */
	FQuicPayloadPtr HandleManagerSerializeHeader(FMessageHeader& MessageHeader);

	/**
	 * Handles message headers that need to be deserialized.
	 */
	FMessageHeader HandleManagerDeserializeHeader(FQuicPayloadPtr HeaderData);

	/**
	 * Handles discovered transport endpoints.
	 */
	void HandleManagerNodeDiscovered(const FGuid& DiscoveredNodeId);

	/**
	 * Handles lost transport endpoints.
	 */
	void HandleManagerNodeLost(const FGuid& LostNodeId);

	/**
	 * Handles client connection changes.
	 */
	void HandleClientConnectionChange(
		const FGuid& NodeId, const FIPv4Endpoint& RemoteEndpoint,
		const EQuicClientConnectionChange ConnectionState);

private:

	/** Holds any pending restart requests. */
	TFuture<void> ErrorFuture;

	/** Holds the transport error delegate. */
	FOnError TransportErrorDelegate;

	/** Holds the message processor. */
	TUniquePtr<FQuicMessageProcessor> MessageProcessor = nullptr;

	/** Message transport handler. */
	IMessageTransportHandler* TransportHandler = nullptr;

	/** Holds the endpoint manager. */
	TUniquePtr<FQuicEndpointManager> EndpointManager = nullptr;

	/** Flag indicating the endpoint mode. */
	EEndpointMode EndpointMode = EEndpointMode::Client;

	/** Holds the QuicEndpoint config. */
	TSharedRef<FQuicEndpointConfig> EndpointConfig;

	/** Holds the static endpoints. */
	TSet<FIPv4Endpoint> StaticEndpoints;

private:

	/** Current id for serialized messages. */
	uint32 SerializedMessageId = 0;

	/** Holds serialized messages. */
	TMap<uint32, TSharedPtr<FQuicSerializedMessage, ESPMode::ThreadSafe>> SerializedMessages;

	/** Holds the meta message received delegate. */
	FOnQuicMetaMessageReceived OnQuicMetaMessageDelegate;

	/** Holds the client connection changed delegate. */
	FOnQuicClientConnectionChanged OnQuicClientConnectionChangedDelegate;

};
