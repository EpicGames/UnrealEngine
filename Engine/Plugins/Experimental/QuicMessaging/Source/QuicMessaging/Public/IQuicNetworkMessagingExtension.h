// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "INetworkMessagingExtension.h"

#include "QuicTransportMessages.h"
#include "QuicTransportNotifications.h"


/**
 * Interface for the messaging module network extension
 * Plugins or modules implementing messaging transport for MessageBus
 * can implement this modular feature to provide control on the service it provides.
 */
class IQuicNetworkMessagingExtension : public INetworkMessagingExtension
{

public:

	/**
	 * Get the generated Guid for this instance.
	 * 
	 * @return The Guid
	 */
	virtual FGuid GetEndpointGuid() = 0;

	/**
	 * Set the maximum size an authentication message can have.
	 *
	 * @param MaxBytes The maximum message size in bytes
	 */
	virtual void SetMaxAuthenticationMessageSize(const uint32 MaxBytes) = 0;

	/**
	 * Get the NodeId of the remote endpoint.
	 *
	 * @param RemoteEndpoint The remote endpoint address:port
	 *
	 * @return Optional FGuid if the node was found
	 */
	virtual TOptional<FGuid> GetNodeId(const FString& RemoteEndpoint) = 0;

	/**
	 * Check if this node is authenticated.
	 * 
	 * @param NodeId The node identifier
	 * @return Whether the node is authenticated
	 */
	virtual bool IsNodeAuthenticated(const FGuid& NodeId) const = 0;

	/**
	 * Set this node as authenticated.
	 * 
	 * @param NodeId The node identifier
	 */
	virtual void SetNodeAuthenticated(const FGuid& NodeId) = 0;

	/**
	 * Disconnect a node.
	 *
	 * @param NodeId The node identifier
	 */
	virtual void DisconnectNode(const FGuid& NodeId) = 0;

	/**
	 * Set the connection cooldown.
	 *
	 * @param bEnabled Whether the cooldown should be enabled
	 * @param MaxAttempts Maximum number of attempts
	 * @param PeriodSeconds Time period for maximum number of attempts in seconds
	 * @param CooldownSeconds Duration of the cooldown in seconds
	 * @param CooldownMaxSeconds Max duration of the cooldown in seconds
	 */
	virtual void SetConnectionCooldown(
		const bool bEnabled, const uint32 MaxAttempts, const uint32 PeriodSeconds,
		const uint32 CooldownSeconds, const uint32 CooldownMaxSeconds) = 0;

	/**
	 * Transport authentication message to endpoint.
	 *
	 * @param Context The message context
	 * @param Recipient Guid of message recipient
	 *
	 * @return Whether the message was sent
	 */
	virtual bool TransportAuthMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    	const FGuid& Recipient) = 0;

	/**
	 * Transport authentication response message to endpoint.
	 *
	 * @param Context The message context
	 * @param Recipient Guid of message recipient
	 *
	 * @return Whether the message was sent
	 */
	virtual bool TransportAuthResponseMessage(
		const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context,
    	const FGuid& Recipient) = 0;

	/**
	 * Delegate that notifies bound functions when
	 * a meta / authentication message has been received.
	 *
	 * @return The delegate
	 */
	virtual FOnQuicMetaMessageReceived& OnQuicMetaMessageReceived() = 0;

	/**
	 * Delegate that notifies bound functions when a QuicClient
	 * has connected to or disconnected from a remote endpoint (QuicServer).
	 *
	 * @return The delegate
	 */
	virtual FOnQuicClientConnectionChanged& OnQuicClientConnectionChanged() = 0;

};
