// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "IMessageContext.h"

#include "QuicTransportMessages.generated.h"


DECLARE_MULTICAST_DELEGATE_TwoParams(FOnQuicMetaMessageReceived,
	const FGuid /*LocalNodeId*/, const TSharedRef<IMessageContext>& /*Context*/);


USTRUCT()
struct FQuicMetaMessage
{

	GENERATED_BODY()

    UPROPERTY(VisibleAnywhere, Category = "QuicTransportMessage")
    FGuid SenderNodeId;

};


USTRUCT()
struct FQuicAuthMessage : public FQuicMetaMessage
{

	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "QuicTransportMessage")
	FString Payload;

};


USTRUCT()
struct FQuicAuthResponseMessage : public FQuicMetaMessage
{

	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = "QuicTransportMessage")
	bool bAuthSuccessful;

	UPROPERTY(VisibleAnywhere, Category = "QuicTransportMessage")
	FString Reason;

};


class FQuicMetaMessageContext
	: public IMessageContext
{

public:

	template <typename MessageType>
	explicit FQuicMetaMessageContext(MessageType* InMessage)
		: Message(InMessage)
		, TypeInfo(InMessage->StaticStruct())
	{ }

	~FQuicMetaMessageContext()
	{
		if (Message)
		{
			if (TypeInfo.IsValid())
			{
				TypeInfo->DestroyStruct(Message);
			}

			// @note: Due to the nature of how message contexts work,
			// there is no other way to do this than to free the message memory.
			FMemory::Free(Message);
		}
	}

public:

	//~ IMessageContext interface

	virtual const TMap<FName, FString>& GetAnnotations() const override { return Annotations; }
	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const override { return Attachment; }
	virtual const FDateTime& GetExpiration() const override { return Expiration; }
	virtual const void* GetMessage() const override { return Message; }
	virtual const TWeakObjectPtr<UScriptStruct>& GetMessageTypeInfo() const override { return TypeInfo; }
	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const override { return OriginalContext; }
	virtual const TArray<FMessageAddress>& GetRecipients() const override { return Recipients; }
	virtual EMessageScope GetScope() const override { return Scope; }
	virtual EMessageFlags GetFlags() const override { return Flags; }
	virtual const FMessageAddress& GetSender() const override { return Sender; }
	virtual const FMessageAddress& GetForwarder() const override { return Sender; }
	virtual ENamedThreads::Type GetSenderThread() const override { return SenderThread; }
	virtual const FDateTime& GetTimeForwarded() const override { return TimeSent; }
	virtual const FDateTime& GetTimeSent() const override { return TimeSent; }

private:

	TMap<FName, FString> Annotations = TMap<FName, FString>();
	TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> Attachment = nullptr;
	FDateTime Expiration = FDateTime::MaxValue();
	void* Message;
	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> OriginalContext = nullptr;
	TArray<FMessageAddress> Recipients = TArray<FMessageAddress>();
	EMessageScope Scope = EMessageScope::Network;
	EMessageFlags Flags = EMessageFlags::Reliable;
	FMessageAddress Sender = FMessageAddress::NewAddress();
	ENamedThreads::Type SenderThread = ENamedThreads::AnyThread;
	FDateTime TimeSent = FDateTime::Now();
	TWeakObjectPtr<UScriptStruct> TypeInfo;

};

