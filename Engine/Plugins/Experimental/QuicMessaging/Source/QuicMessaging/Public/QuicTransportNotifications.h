// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Delegates/Delegate.h"

struct FGuid;
struct FIPv4Endpoint;


/**
 * Enumerates QuicClient connection state to the remote endpoint.
 */
enum class EQuicClientConnectionState : uint8
{
	/** Client connected to remote endpoint. */
	Connected,

	/** Client disconnected from remote endpoint. */
	Disconnected
};


/**
 * Delegate informing bound functions when a QuicClient
 * has connected to or disconnected from a remote endpoint (QuicServer).
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnQuicClientConnectionChanged,
	const FGuid& /*NodeId*/, const FIPv4Endpoint& /*RemoteEndpoint*/,
	const EQuicClientConnectionState /*ConnectionState*/);

