// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "QuicMessagingSettings.generated.h"

/** Defines the Quic message format available (how the message data is encoded). */
UENUM()
enum class EQuicMessageFormat : uint8
{
	/** No format specified. Legacy - Not exposed to user.*/
	None = 0 UMETA(Hidden),

	/** JSON format specified. Legacy - Not exposed to user.*/
	Json UMETA(Hidden),

	/** Tagged property format specified. Legacy - Not exposed to user.*/
	TaggedProperty UMETA(Hidden),

	/**
	 * Quic messages are encoded in CBOR, using the platform endianness. This is the fastest and preferred option, but the CBOR data will not be readable by an external standard-compliant CBOR parser
	 * if generated from a little endian platform. If the data needs to be consumed outside the Unreal Engine, consider using CborStandardEndianness format instead.
	 */
	CborPlatformEndianness UMETA(DisplayName="CBOR (Platform Endianness)"),

	/**
	 * Quic messages are encoded in CBOR, using the CBOR standard-complinant endianness (big endian). It will perform slower on a little-endian platform, but the data will be readable by standard CBOR parsers.
	 * Useful if the Quic messages needs to be analyzed/consumed outside the Unreal Engine.
	 */
	CborStandardEndianness UMETA(DisplayName="CBOR (Standard Endianness)"),
};

UCLASS(config=Engine)
class QUICMESSAGING_API UQuicMessagingSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Whether Quic messaging is enabled by default. If false -messaging will need to be added
	 * to the commandline when running non-editor builds.
	 *
	 * (Note - in Shipping builds ALLOW_QUIC_MESSAGING_SHIPPING=1 must also be defined in TargetRules
	 * for messaging to be available with or without this setting)
	 */
	UPROPERTY(config, EditAnywhere, Category = Availability)
	bool EnabledByDefault = false;

	/**
	 * Whether the Quic transport channel is enabled.
	 * Can be specified on the command line with `-QUICMESSAGING_TRANSPORT_ENABLE=`
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	bool EnableTransport = true;

	/** Whether the Quic transport channel should try to auto repair when in error. */
	UPROPERTY(config, EditAnywhere, Category=Transport, AdvancedDisplay)
	bool bAutoRepair = true;

	/**
	 * Whether the QUIC transport endpoint is a client (true) or a server (false)
	 */
	UPROPERTY(config)
	bool bIsClient = true;

	/**
	 * Whether encryption should be used after the quic handshake
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	bool bEncryption = true;

	/**
	 * Timeout in seconds when the remote endpoint cannot be discovered.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 DiscoveryTimeoutSeconds = 10;

	/**
	 * Whether server side authentication is enabled
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	bool bAuthEnabled = false;

	/**
	 * Maximum authentication message size for server side authentication
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 MaxAuthenticationMessageSize = 8000;

	/**
	 * Whether server has a cooldown for connection attempts.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	bool bConnectionCooldownEnabled = true;

	/**
	 * Maximum number of connection attempts until the cooldown is triggered.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 ConnectionCooldownMaxAttempts = 5;

	/**
	 * Time period in seconds within which the maximum attempts must happen.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 ConnectionCooldownPeriodSeconds = 30;

	/**
	 * Connection cooldown in seconds used with exponential backoff.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 ConnectionCooldownSeconds = 60;

	/**
	 * Maximum connection cooldown in seconds for exponential backoff.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 ConnectionCooldownMaxSeconds = 3600;

	/**
	 * Whether the QUIC client should verify the server certificate
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	bool bClientVerificationEnabled = true;

	/**
	 * Server certificate to encrypt QUIC transport
	 */
	UPROPERTY(config)
	FString QuicServerCertificate = "";

	/**
	 * Server private key to encrypt QUIC transport
	 */
	UPROPERTY(config)
	FString QuicServerPrivateKey = "";

	/**
	 * Guid of this endpoint
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	FGuid EndpointGuid;

	/** The number of consecutive attempt the auto repair routine will try to repair. */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 AutoRepairAttemptLimit = 10;

	/** Whether to stop the transport service when the application deactivates, and restart it when the application is reactivated */
	UPROPERTY(config)
	bool bStopServiceWhenAppDeactivates = true;

	/**
	 * The IP endpoint to listen to and send packets from.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 * 0.0.0.0:0 will bind to the default network adapter on Windows,
	 * and all available network adapters on other operating systems.
	 * Specifying an interface IP here, will use that interface for multicasting and static endpoint *might* also reach this client through <unicast ip:multicast port>
	 * Specifying both the IP and Port will allow usage of static endpoint to reach this client
	 * Can be specified on the command line with `-QUICMESSAGING_TRANSPORT_UNICAST=`
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	FString UnicastEndpoint;

	/** The format used to serialize the Quic message payload. */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	EQuicMessageFormat MessageFormat = EQuicMessageFormat::CborPlatformEndianness;

	/**
	 * The IP endpoints of static devices.
	 *
	 * Use this setting to reach devices on other subnets, such as mobile phones on a WiFi network.
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport, AdvancedDisplay)
	TArray<FString> StaticEndpoints;

};
