// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class QuicMessaging : ModuleRules
	{
		public QuicMessaging(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"QuicMessagingTransport"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"Json",
					"JsonUtilities",
					"Cbor",
					"Messaging",
					"Networking",
					"Serialization",
					"TraceLog",
					"Sockets",
					"Projects",
					"MsQuicRuntime"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"MessagingCommon",
					"MsQuicRuntime"
				}
			);

			PrivateIncludePaths.AddRange(
				new string[] {
					"QuicMessaging/Private/Transport",
					"QuicMessaging/Private/Serialization",
					"QuicMessaging/Public/Shared",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
				}
			);

			if (Target.bCompileAgainstEditor)
			{

				DynamicallyLoadedModuleNames.AddRange(
					new string[] {
						"Settings",
					}
				);

				PrivateIncludePathModuleNames.AddRange(
					new string[] {
						"Settings",
					}
				);
			}
		}
	}
}
