// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "QuicIncludes.h"
#include "QuicMessages.h"
#include "Templates/SharedPointer.h"

struct FMessageHeader;

using FQuicPayloadPtr = TSharedPtr<TArray<uint8>, ESPMode::ThreadSafe>;


struct FInboundBuffer
{
	/** Flag indicating whether the header was deserialized. */
	bool bHeaderDeserialized;

	/** Holds the Quic handler address. */
	QUIC_ADDR HandlerAddress;

	/** Holds the Quic handler endpoint. */
	FIPv4Endpoint HandlerEndpoint;

	/** Holds the message header. */
	FMessageHeader MessageHeader;

	/** Holds the inbound data. */
	FQuicPayloadPtr InboundData;

	/** Default constructor. */
	FInboundBuffer()
		: bHeaderDeserialized(false)
		, HandlerAddress(QUIC_ADDR())
		, HandlerEndpoint(FIPv4Endpoint::Any)
		, MessageHeader(FMessageHeader())
		, InboundData(nullptr)
	{ }

	/** Creates and initializes a new instance. */
	FInboundBuffer(const QUIC_ADDR InHandlerAddress,
		const FIPv4Endpoint InHandlerEndpoint, const FQuicPayloadPtr InInboundData)
		: bHeaderDeserialized(false)
		, HandlerAddress(InHandlerAddress)
		, HandlerEndpoint(InHandlerEndpoint)
		, MessageHeader(FMessageHeader())
		, InboundData(InInboundData)
	{ }
};

struct FInboundQuicBuffer
{
	/** Holds the Quic stream ID. */
	uint64 StreamId;

	/** Holds the Quic stream handle. */
	HQUIC Stream;

	/** Flag indicating whether this is the end of the Quic stream. */
	bool bEndOfStream;

	/** Holds the peer address. */
	FIPv4Endpoint PeerAddress;

	/** Holds the total length of all buffers. */
	uint64 BufferLength;

	/** Holds buffer pointers and buffer sizes. */
	TMap<uint8*, uint32> Data;

	/** Default constructor. */
	FInboundQuicBuffer()
		: StreamId(0)
		, Stream(nullptr)
		, bEndOfStream(false)
		, PeerAddress(FIPv4Endpoint::Any)
		, BufferLength(0)
		, Data(TMap<uint8*, uint32>())
	{ }

	/** Creates and initializes a new instance. */
	FInboundQuicBuffer(const uint64 InStreamId, HQUIC InStream,
		const bool bInEndOfStream, const FIPv4Endpoint InPeerAddress,
		const uint64 InBufferLength)
		: StreamId(InStreamId)
		, Stream(InStream)
		, bEndOfStream(bInEndOfStream)
		, PeerAddress(InPeerAddress)
		, BufferLength(InBufferLength)
		, Data(TMap<uint8*, uint32>())
	{ }
};

