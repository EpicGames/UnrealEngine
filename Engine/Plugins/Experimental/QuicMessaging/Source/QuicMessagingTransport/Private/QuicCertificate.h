// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QuicIncludes.h"

#include "Containers/StringConv.h"
#include "HAL/FileManagerGeneric.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Templates/Tuple.h"

namespace QuicCertificateUtils
{

	/**
	 * Helper to setup a QUIC credential config.
	 */
	typedef struct QUIC_CREDENTIAL_CONFIG_HELPER
	{
		QUIC_CREDENTIAL_CONFIG CredConfig;

		QUIC_CERTIFICATE_FILE CertificateFile;

	} QUIC_CREDENTIAL_CONFIG_HELPER;


	/**
	 * Generate a 2048-bit RSA key.
	 *
	 * @param PrivateKey The private key struct
	 */
	inline bool GeneratePrivateKey(EVP_PKEY* PrivateKey)
	{
		if (!PrivateKey)
		{
			return false;
		}

		/* Generate the RSA key and assign it to pkey. */
		RSA* Rsa = RSA_new();

		{
			BIGNUM* BigNum = BN_new();
			BN_set_word(BigNum, RSA_F4);
			RSA_generate_key_ex(Rsa, 2048, BigNum, nullptr);
			BN_free(BigNum);
		}

		if (!EVP_PKEY_assign_RSA(PrivateKey, Rsa))
		{
			EVP_PKEY_free(PrivateKey);
			return false;
		}

		return true;
	}


	/**
	 * Converts a private key to a string.
	 *
	 * @param PrivateKey The private key struct
	 */
	inline char* PrivateKeyToString(EVP_PKEY* PrivateKey)
	{
		BIO* Bio = BIO_new(BIO_s_mem());

		int Ret = PEM_write_bio_PrivateKey(Bio, PrivateKey, NULL, NULL, 0, NULL, NULL);

		if (Ret != 1)
		{
			BIO_free(Bio);
		}

		BIO_flush(Bio);

		char* Text;
		long Length = BIO_get_mem_data(Bio, &Text);
		char* Result = new char[Length + 1];
		FMemory::Memcpy(Result, Text, Length);
		Result[Length] = 0;

		BIO_free(Bio); // text is just a pointer to bio's data, no need to free that as well

		return Result;
	}


	/**
	 * Generate self-signed x509 certificate.
	 *
	 * @param PrivateKey The private key struct
	 * @param x509 The certificate struct
	 */
	inline bool GenerateCertificate(EVP_PKEY* PrivateKey, X509* x509)
	{
		if (!x509)
		{
			return false;
		}

		/* Set the serial number. */
		ASN1_INTEGER_set(X509_get_serialNumber(x509), 1);

		/* This certificate is valid from now until exactly one year from now. */
		X509_gmtime_adj(X509_get_notBefore(x509), 0);
		X509_gmtime_adj(X509_get_notAfter(x509), 31536000L);

		/* Set the public key for our certificate. */
		X509_set_pubkey(x509, PrivateKey);

		/* We want to copy the subject name to the issuer name. */
		X509_NAME* SubjectName = X509_get_subject_name(x509);

		/* Set the country code and common name. */
		X509_NAME_add_entry_by_txt(SubjectName, "C", MBSTRING_ASC, (unsigned char*)"CA", -1, -1, 0);
		X509_NAME_add_entry_by_txt(SubjectName, "O", MBSTRING_ASC, (unsigned char*)"MyCompany", -1, -1, 0);
		X509_NAME_add_entry_by_txt(SubjectName, "CN", MBSTRING_ASC, (unsigned char*)"localhost", -1, -1, 0);

		/* Now set the issuer name. */
		X509_set_issuer_name(x509, SubjectName);

		/* Actually sign the certificate with our key. */
		if (!X509_sign(x509, PrivateKey, EVP_sha1()))
		{
			//std::cerr << "Error signing certificate." << std::endl;
			X509_free(x509);
			return false;
		}

		return true;
	}


	/**
	 * Convert a certificate to a string.
	 *
	 * @param x509 The certificate struct
	 */
	inline char* X509ToString(X509* x509)
	{
		BIO* Bio = BIO_new(BIO_s_mem());
		int Ret = PEM_write_bio_X509(Bio, x509);

		char* Text;
		long Length = BIO_get_mem_data(Bio, &Text);
		char* Result = new char[Length + 1];
		FMemory::Memcpy(Result, Text, Length);
		Result[Length] = 0;

		BIO_free(Bio); // text is just a pointer to bio's data, no need to free that as well
		return Result;
	}


	/**
	 * Get the paths for self-signed certificate and key.
	 */
	inline TTuple<FString, FString> GetSelfSignedPaths()
	{
		IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();

		FString EngineCertificates = FileManager.ConvertToAbsolutePathForExternalAppForWrite(
			*FPaths::EngineContentDir()) + TEXT("Certificates");

		FString CertificatesDir = EngineCertificates + TEXT("/QuicMessaging");

		FString CertificateFile = CertificatesDir + TEXT("/transportcert.pem");
		FString PrivateKeyFile = CertificatesDir + TEXT("/transportkey.key");

		return TTuple<FString, FString>(CertificateFile, PrivateKeyFile);
	}


	/**
	 * Generates a self-signed certificate and returns paths to (certificate, private key).
	 */
	inline bool CreateSelfSigned()
	{
		/* Allocate memory for the EVP_PKEY structure. */
		EVP_PKEY* PrivateKey = EVP_PKEY_new();

		if (!GeneratePrivateKey(PrivateKey))
		{
			return false;
		}

		/* Allocate memory for the X509 structure. */
		X509* x509 = X509_new();

		if (!GenerateCertificate(PrivateKey, x509))
		{
			return false;
		}

		TTuple<FString, FString> Paths = GetSelfSignedPaths();

		FString CertificateFile = Paths.Key;
		FString PrivateKeyFile = Paths.Value;

		FString CertificateString = ANSI_TO_TCHAR(X509ToString(x509));
		FString PrivateKeyString = CertificateString + ANSI_TO_TCHAR(PrivateKeyToString(PrivateKey));

		FFileHelper::SaveStringToFile(CertificateString, *CertificateFile);
		FFileHelper::SaveStringToFile(PrivateKeyString, *PrivateKeyFile);

		X509_free(x509);
		EVP_PKEY_free(PrivateKey);

		return true;
	}

};

