// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicClient.h"


void FQuicClient::StartEndpoint()
{
	QUIC_STATUS Status;

	/**
	 * Allocate new connection object
	 */
	if (QUIC_FAILED(Status = MsQuic->ConnectionOpen(Registration,
		ClientConnectionCallback, this, &ClientConnection)))
	{
		ConnectionState = EQuicClientState::Failed;
		EndpointError = QuicUtils::ConvertQuicStatus(Status);

		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Could not open connection: %ls.",
			*QuicUtils::GetEndpointErrorString(EndpointError));

		return;
	}

	if (ClientConfig->EncryptionMode == EEncryptionMode::Disabled)
	{
		constexpr BOOLEAN Value = true;

		MsQuic->SetParam(ClientConnection,
			QUIC_PARAM_CONN_DISABLE_1RTT_ENCRYPTION,
			sizeof(Value), &Value);
	}

	/**
	 * Start connection to the server
	 */
	if (QUIC_FAILED(Status = MsQuic->ConnectionStart(ClientConnection,
		QuicConfig, QUIC_ADDRESS_FAMILY_INET,
		StringCast<ANSICHAR>(*Endpoint.Address.ToString()).Get(), Endpoint.Port)))
	{
		ConnectionState = EQuicClientState::Failed;
		EndpointError = QuicUtils::ConvertQuicStatus(Status);

		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Could not start connection: %ls.",
			*QuicUtils::GetEndpointErrorString(EndpointError));

		return;
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, "[QuicClient] Started client.");
}


void FQuicClient::StopEndpoint()
{
	if (IsStopping())
	{
		return;
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, "[QuicClient] Stopping client ...");

	Stop();

	if (ConnectionState == EQuicClientState::Connecting)
	{
		ConnectionState = EQuicClientState::Disconnected;
	}

	if (MsQuic)
	{
		if (ClientConnection)
		{
			// Only do this if the connection wasn't already shutdown/closed by the peer or transport
			if (ConnectionState != EQuicClientState::Stopped)
			{
				if ((EndpointError == EQuicEndpointError::Normal
					|| EndpointError == EQuicEndpointError::ConnectionAbort)
					&& ConnectionState != EQuicClientState::Closed)
				{
					MsQuic->ConnectionShutdown(ClientConnection, QUIC_CONNECTION_SHUTDOWN_FLAG_NONE, 0);
					MsQuic->ConnectionClose(ClientConnection);
				}

				ConnectionState = EQuicClientState::Stopped;
			}
		}
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, "[QuicClient] Stopped client.");
}


bool FQuicClient::IsEndpointReady()
{
	return ConnectionState == EQuicClientState::Connected;
}


void FQuicClient::UpdateEndpoint()
{
	if (IsStopping() || ConnectionState != EQuicClientState::Connected)
	{
		return;
	}

	ConsumeMessages();
}


void FQuicClient::ConsumeInboundMessages()
{
	if (InboundMessages.IsEmpty())
	{
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	const double ConsumeLimit = CurrentTime + MaxConsumeTime.GetTotalSeconds();

	FInboundMessage InboundMessage;

	while (CurrentTime < ConsumeLimit && InboundMessages.Dequeue(InboundMessage))
	{
		MessageInboundDelegate.ExecuteIfBound(InboundMessage);

		CurrentTime = FPlatformTime::Seconds();
	}
}


void FQuicClient::ProcessInboundBuffers()
{
	if (InboundQuicBuffers.IsEmpty())
	{
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	const double ConsumeLimit = CurrentTime + MaxConsumeTime.GetTotalSeconds();

	FInboundQuicBuffer InboundBuffer;

	while (CurrentTime < ConsumeLimit && InboundQuicBuffers.Dequeue(InboundBuffer))
	{
		if (ReceiveFromStream(InboundBuffer))
		{
			if (InboundBuffer.bEndOfStream)
			{
				ReachedEndOfStream(InboundBuffer);
			}
		}

		CurrentTime = FPlatformTime::Seconds();
	}
}


void FQuicClient::ConsumeOutboundMessages()
{
	if (OutboundMessages.IsEmpty())
	{
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	const double ConsumeLimit = CurrentTime + MaxConsumeTime.GetTotalSeconds();

	FOutboundMessage OutboundMessage;

	while (CurrentTime < ConsumeLimit && OutboundMessages.Dequeue(OutboundMessage))
	{
		SendToStream(OutboundMessage);

		CurrentTime = FPlatformTime::Seconds();
	}
}


void FQuicClient::CollectStatistics()
{
	if (IsStopping() || ConnectionState != EQuicClientState::Connected)
	{
		return;
	}

	if (MsQuic == nullptr)
	{
		return;
	}

	QUIC_STATISTICS Stats{};

	QUIC_STATUS Status;
	
	if (QUIC_FAILED(Status = GetConnectionStats(ClientConnection, Stats)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Could not collect statistics: %ls.",
			*QuicUtils::ConvertResult(Status));

		return;
	}

	FMessageTransportStatistics TransportStats = ConvertStatistics(Endpoint, Stats);

	StatisticsUpdatedDelegate.ExecuteIfBound(TransportStats);
}


void FQuicClient::EnqueueOutboundMessage(const FOutboundMessage& OutboundMessage)
{
	OutboundMessages.Enqueue(OutboundMessage);
}


void FQuicClient::SendHello(FIPv4Endpoint RemoteEndpoint)
{
	if (IsStopping() || ConnectionState != EQuicClientState::Connected)
	{
		return;
	}

	FMessageHeader MessageHeader(EQuicMessageType::Hello, FGuid(), LocalNodeId, 0);
	const FQuicPayloadPtr MetaBuffer = SerializeHeaderDelegate.Execute(MessageHeader);

	const FOutboundMessage OutboundMessage(RemoteEndpoint, MetaBuffer);

	EnqueueOutboundMessage(OutboundMessage);

	UE_LOGF(LogQuicMessagingTransport, Verbose, "[QuicClient] Sent hello.");
}


void FQuicClient::SendBye()
{
	if (IsStopping() || ConnectionState != EQuicClientState::Connected)
	{
		return;
	}

	FMessageHeader MessageHeader(EQuicMessageType::Bye, FGuid(), LocalNodeId, 0);
	const FQuicPayloadPtr MetaBuffer = SerializeHeaderDelegate.Execute(MessageHeader);
	
	const FOutboundMessage OutboundMessage(Endpoint, MetaBuffer);

	SendToStream(OutboundMessage);

	UE_LOGF(LogQuicMessagingTransport, Verbose, "[QuicClient] Sent bye.");
}


_Function_class_(QUIC_CONNECTION_CALLBACK)
QUIC_STATUS FQuicClient::ClientConnectionCallback(HQUIC Connection,
	void* Context, QUIC_CONNECTION_EVENT* Event)
{
	FQuicClient* ClientContext = static_cast<FQuicClient*>(Context);

	switch (Event->Type)
	{

	case QUIC_CONNECTION_EVENT_CONNECTED:

		/**
		 * Connection handshake has completed
		 */

		ClientContext->OnConnectionConnected();

		break;

	case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_TRANSPORT:

		/**
		 * Idle timeout killed the connection
		 */

		ClientContext->OnConnectionShutdownByTransport(Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status);

		break;

	case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_PEER:

		/**
		 * Peer killed the connection
		 */

		ClientContext->OnConnectionShutdownByPeer(Event->SHUTDOWN_INITIATED_BY_PEER.ErrorCode);

		break;

	case QUIC_CONNECTION_EVENT_SHUTDOWN_COMPLETE:

		/**
		 * Connection shutdown complete, ready to cleanup
		 */

		ClientContext->OnConnectionShutdownComplete(Connection, Event->SHUTDOWN_COMPLETE.AppCloseInProgress);

		break;

	case QUIC_CONNECTION_EVENT_RESUMPTION_TICKET_RECEIVED:

		/**
		 * Resumption ticket received. Does not do anything at this time
		 */

		ClientContext->OnConnectionResumptionTicketReceived();

		break;

	case QUIC_CONNECTION_EVENT_PEER_STREAM_STARTED:
	{
		/**
		 * Peer started a stream, set local stream callback handler
		 */

		FIPv4Endpoint* PeerAddress = new FIPv4Endpoint(ClientContext->Endpoint);

		FStreamCallbackInfo* StreamCallbackInfo = new FStreamCallbackInfo();
		StreamCallbackInfo->Endpoint = ClientContext;
		StreamCallbackInfo->PeerAddress = PeerAddress;

		ClientContext->MsQuic->SetCallbackHandler(Event->PEER_STREAM_STARTED.Stream,
			reinterpret_cast<void*>(ClientStreamCallback), StreamCallbackInfo);

		ClientContext->OnConnectionPeerStreamStarted(Event->PEER_STREAM_STARTED.Stream);

		break;
	}

	case QUIC_CONNECTION_EVENT_DATAGRAM_RECEIVED:
	case QUIC_CONNECTION_EVENT_DATAGRAM_SEND_STATE_CHANGED:
	case QUIC_CONNECTION_EVENT_DATAGRAM_STATE_CHANGED:
	case QUIC_CONNECTION_EVENT_IDEAL_PROCESSOR_CHANGED:
	case QUIC_CONNECTION_EVENT_LOCAL_ADDRESS_CHANGED:
	case QUIC_CONNECTION_EVENT_PEER_ADDRESS_CHANGED:
	case QUIC_CONNECTION_EVENT_PEER_CERTIFICATE_RECEIVED:
	case QUIC_CONNECTION_EVENT_PEER_NEEDS_STREAMS:
	case QUIC_CONNECTION_EVENT_RESUMED:
	case QUIC_CONNECTION_EVENT_STREAMS_AVAILABLE:

		break;

	default:

		UE_LOGF(LogQuicMessagingTransport, Warning,
			"[QuicClient] Received unhandled connection event: %d", Event->Type);

		break;

	}

	return QUIC_STATUS_SUCCESS;
}


_Function_class_(QUIC_STREAM_CALLBACK)
QUIC_STATUS FQuicClient::ClientStreamCallback(HQUIC Stream,
	void* Context, QUIC_STREAM_EVENT* Event)
{
	const FStreamCallbackInfo* CallbackInfo = static_cast<FStreamCallbackInfo*>(Context);
	FQuicClient* ClientContext = static_cast<FQuicClient*>(CallbackInfo->Endpoint);

	switch (Event->Type)
	{

	case QUIC_STREAM_EVENT_SEND_COMPLETE:

		/**
		 * Previous StreamSend has completed, context returned
		 */

		ClientContext->OnStreamSendComplete(Stream, Event);

		break;

	case QUIC_STREAM_EVENT_RECEIVE:
	
		/**
		 * Data received from the peer
		 */

		ClientContext->OnStreamReceive(Stream, *CallbackInfo->PeerAddress, Event);

		return QUIC_STATUS_PENDING;
	

	case QUIC_STREAM_EVENT_PEER_SEND_ABORTED:

		/**
		 * Peer aborted its stream send direction
		 */

		ClientContext->OnStreamPeerSendAborted(Stream);

		break;

	case QUIC_STREAM_EVENT_PEER_SEND_SHUTDOWN:

		/**
		 * Peer gracefully shut down its stream send direction
		 */

		break;

	case QUIC_STREAM_EVENT_SHUTDOWN_COMPLETE:
	
		/**
		 * Both stream directions have been shut down, safe to cleanup
		 */

		ClientContext->OnStreamShutdownComplete(Stream);
		delete CallbackInfo;

		break;

	case QUIC_STREAM_EVENT_START_COMPLETE:
	case QUIC_STREAM_EVENT_PEER_ACCEPTED:
	case QUIC_STREAM_EVENT_PEER_RECEIVE_ABORTED:
	case QUIC_STREAM_EVENT_IDEAL_SEND_BUFFER_SIZE:
	case QUIC_STREAM_EVENT_SEND_SHUTDOWN_COMPLETE:

		break;

	default:

		UE_LOGF(LogQuicMessagingTransport, Warning,
			"[QuicClient] Received unhandled stream event: %d", Event->Type);

		break;

	}

	return QUIC_STATUS_SUCCESS;
}


void FQuicClient::OnConnectionConnected()
{
	ConnectionState = EQuicClientState::Connected;

	EndpointConnectedDelegate.ExecuteIfBound(Endpoint);

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicClient] Handshake completed. Server: %ls / LocalNodeId: %ls / ThreadId: %d.",
		*Endpoint.ToString(), *LocalNodeId.ToString(), Thread->GetThreadID());
}


void FQuicClient::OnConnectionShutdownByTransport(QuicUtils::HRESULT Status)
{
	ConnectionState = EQuicClientState::Disconnected;
	bTransportClosed = true;

	EndpointError = QuicUtils::ConvertQuicStatus(Status);

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicClient] Connection end (by transport). Server: %ls.",
		*Endpoint.ToString());

	if (EndpointError != EQuicEndpointError::Normal)
	{
		UE_LOGF(LogQuicMessagingTransport, Warning,
			"[QuicClient] Shutdown reason: %ls",
			*QuicUtils::GetEndpointErrorString(EndpointError));
	}
}


void FQuicClient::OnConnectionShutdownByPeer(QUIC_UINT62 ErrorCode)
{
	ConnectionState = EQuicClientState::Disconnected;
	bTransportClosed = true;

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicClient] Connection end (by peer). Server: %ls / Error code: %lld.",
		*Endpoint.ToString(), ErrorCode);
}


void FQuicClient::OnConnectionShutdownComplete(HQUIC Connection, BOOLEAN AppCloseInProgress)
{
	if (!AppCloseInProgress)
	{
		UE_LOGF(LogQuicMessagingTransport, Verbose,
			"[QuicClient] Connection shutdown complete. Server: %ls.",
			*Endpoint.ToString());

		if (MsQuic && ConnectionState != EQuicClientState::Stopped)
		{
			if (ConnectionState != EQuicClientState::Closed)
			{
				MsQuic->ConnectionClose(Connection);
				ConnectionState = EQuicClientState::Closed;
			}

			EndpointDisconnectedDelegate.ExecuteIfBound(Endpoint, EndpointError);
		}
	}
}


void FQuicClient::OnConnectionResumptionTicketReceived()
{
	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicClient] Resumption ticket received.");
}


void FQuicClient::OnConnectionPeerStreamStarted(HQUIC Stream)
{
	const uint64 StreamId = GetStreamId(Stream);

	{
		const FQuicPayloadPtr InboundData = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();

		FScopeLock InboundLock(&InboundBuffersCS);

		const QUIC_ADDR LocalAddress = GetAddressFromEndpoint(Endpoint);
		const FInboundBuffer InboundBuffer(LocalAddress, FIPv4Endpoint::Any, InboundData);
		const FAddrStream AddressStreamKey(Endpoint, StreamId);

		InboundBuffers.Add(AddressStreamKey, InboundBuffer);
	}

	UE_LOGF(LogQuicMessagingTransport, VeryVerbose,
		"[QuicClient] Peer stream started. Server: %ls / StreamId: %lld.",
		*Endpoint.ToString(), StreamId);
}


void FQuicClient::OnStreamShutdownComplete(HQUIC Stream) const
{
	if (MsQuic)
	{
		MsQuic->StreamClose(Stream);
	}
}


void FQuicClient::SendToStream(const FOutboundMessage& OutboundMessage)
{
	HQUIC Stream = nullptr;
	QUIC_STATUS Status;

	FIPv4Endpoint* PeerAddress = new FIPv4Endpoint(Endpoint);

	FStreamCallbackInfo* StreamCallbackInfo = new FStreamCallbackInfo();
	StreamCallbackInfo->Endpoint = this;
	StreamCallbackInfo->PeerAddress = PeerAddress;

	/**
	 * Create/allocate a new unidirectional stream
	 */
	if (QUIC_FAILED(Status = MsQuic->StreamOpen(ClientConnection,
		QUIC_STREAM_OPEN_FLAG_UNIDIRECTIONAL, ClientStreamCallback, StreamCallbackInfo, &Stream)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Could not open stream: %ls.", *QuicUtils::ConvertResult(Status));
		return;
	}

	/**
	 * Start the stream
	 */
	if (QUIC_FAILED(Status = MsQuic->StreamStart(Stream, QUIC_STREAM_START_FLAG_NONE)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Could not start stream: %ls.", *QuicUtils::ConvertResult(Status));

		MsQuic->StreamClose(Stream);
		return;
	}

	const uint64 StreamId = GetStreamId(Stream);

	/**
	 * Send buffers over the stream
	 */
	const FQuicBufferRef MetaBuffer = MakeShared<QUIC_BUFFER, ESPMode::ThreadSafe>();
	const FQuicBufferRef MessageBuffer = MakeShared<QUIC_BUFFER, ESPMode::ThreadSafe>();

	MetaBuffer->Buffer = OutboundMessage.SerializedHeader->GetData();
	MetaBuffer->Length = OutboundMessage.SerializedHeader->Num();

	RegisterStreamMessage(StreamId, *PeerAddress,
		!OutboundMessage.bHasPayload, OutboundMessage, MetaBuffer, MessageBuffer);
	
	if (QUIC_FAILED(Status = MsQuic->StreamSend(Stream, &*MetaBuffer, 1,
		OutboundMessage.bHasPayload ? QUIC_SEND_FLAG_NONE : QUIC_SEND_FLAG_FIN, StreamCallbackInfo)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Could not send meta on stream: %ls.", *QuicUtils::ConvertResult(Status));
		
		MsQuic->StreamShutdown(Stream, QUIC_STREAM_SHUTDOWN_FLAG_ABORT, 0);
		return;
	}

	// This message is header-only
	if (!OutboundMessage.bHasPayload)
	{
		return;
	}

	MessageBuffer->Buffer = OutboundMessage.SerializedPayload->GetData();
	MessageBuffer->Length = OutboundMessage.SerializedPayload->Num();

	if (QUIC_FAILED(Status = MsQuic->StreamSend(Stream,
		&*MessageBuffer, 1, QUIC_SEND_FLAG_FIN, StreamCallbackInfo)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Could not send message on stream: %ls.", *QuicUtils::ConvertResult(Status));
		
		MsQuic->StreamShutdown(Stream, QUIC_STREAM_SHUTDOWN_FLAG_ABORT, 0);
	}
}


bool FQuicClient::ReceiveFromStream(const FInboundQuicBuffer& QuicBuffer)
{
	FScopeLock InboundLock(&InboundBuffersCS);
	
	const FAddrStream AddressStreamKey(QuicBuffer.PeerAddress, QuicBuffer.StreamId);

	FInboundBuffer* Buffer = InboundBuffers.Find(AddressStreamKey);

	if (!Buffer)
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Invalid inbound buffer in ReceiveFromStream %lld.",
			QuicBuffer.StreamId);

		return false;
	}

	const FQuicPayloadPtr& InboundData = Buffer->InboundData;

	for (const TPair<uint8*, uint32> InboundBuffer : QuicBuffer.Data)
	{
		Buffer->InboundData->Append(InboundBuffer.Key, InboundBuffer.Value);
	}

	// Retrieve message header
	if (!Buffer->bHeaderDeserialized
		&& InboundData->Num() >= QUIC_MESSAGE_HEADER_SIZE)
	{
		const FQuicPayloadPtr HeaderData = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();
		HeaderData->Append(InboundData->GetData(), QUIC_MESSAGE_HEADER_SIZE);
		const FMessageHeader MessageHeader = DeserializeHeaderDelegate.Execute(HeaderData);

		Buffer->MessageHeader = MessageHeader;
		Buffer->bHeaderDeserialized = true;

		InboundData->RemoveAt(0, QUIC_MESSAGE_HEADER_SIZE);

		const uint32& SerializedMessageSize = MessageHeader.SerializedMessageSize;
		InboundData->Reserve(SerializedMessageSize);
	}

	InboundLock.Unlock();

	if (MsQuic)
	{
		MsQuic->StreamReceiveComplete(QuicBuffer.Stream, QuicBuffer.BufferLength);
	}

	return true;
}


void FQuicClient::ReachedEndOfStream(const FInboundQuicBuffer& QuicBuffer)
{
	FInboundBuffer InboundBuffer;

	{
		const FAddrStream AddressStreamKey(QuicBuffer.PeerAddress, QuicBuffer.StreamId);

		FScopeLock InboundLock(&InboundBuffersCS);

		if (!InboundBuffers.RemoveAndCopyValue(AddressStreamKey, InboundBuffer))
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[QuicClient] Invalid inbound buffer in ReachedEndOfStream.");

			return;
		}
	}

	const FQuicPayloadPtr& InboundData = InboundBuffer.InboundData;
	const FMessageHeader& MessageHeader = InboundBuffer.MessageHeader;

	// Transmitted message size smaller than expected
	if (InboundData->Num() < static_cast<int32>(MessageHeader.SerializedMessageSize))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicClient] Transmitted message size smaller than expected %lld / %d / %d.",
			QuicBuffer.StreamId, InboundData->Num(), MessageHeader.SerializedMessageSize);

		return;
	}

	FInboundMessage InboundMessage(InboundData, MessageHeader,
		Endpoint, Endpoint);

	InboundMessages.Enqueue(InboundMessage);
}

