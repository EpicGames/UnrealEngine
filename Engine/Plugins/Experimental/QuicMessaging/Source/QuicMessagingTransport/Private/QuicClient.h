// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QuicEndpoint.h"
#include "QuicQueues.h"
#include "QuicMessages.h"

#include "QuicUtils.h"


/**
 * Implements the QuicClient.
 */
class FQuicClient
	: public FQuicEndpoint
	, public FQuicQueues
{

public:

	FQuicClient() = delete;

	/**
	 * Creates and initializes a new quic client.
	 */
	FQuicClient(TSharedRef<FQuicClientConfig> InClientConfig,
		HQUIC InQuicConfig, const QUIC_API_TABLE* InMsQuic, HQUIC InRegistration,
		const TCHAR* InThreadName)
		: FQuicEndpoint(InClientConfig, InQuicConfig,
			InMsQuic, InRegistration, InThreadName)
		, ClientConnection(nullptr)
		, ClientConfig(InClientConfig)
	{ }

protected:

	/**
	 * Starts the configured QUIC client.
	 */
	virtual void StartEndpoint() override;

public:

	/**
	 * Stops the QUIC client.
	 */
	virtual void StopEndpoint() override;

	/**
	 * Indicates if the client is connected and ready to send/receive messages.
	 */
	virtual bool IsEndpointReady() override;

public:

	/**
	 * Periodically called to update the endpoint (consume messages).
	 */
	virtual void UpdateEndpoint() override;

protected:

	/**
	 * Dequeue inbound messages until MaxConsumeTime is reached.
	 * Inbound messages are passed to QuicEndpointManager for validation.
	 */
	virtual void ConsumeInboundMessages() override;

	/**
	 * Dequeue inbound buffers until MaxConsumeTime is reached.
	 * Inbound buffers via QUIC are copied into our buffers until the stream end is indicated.
	 */
	virtual void ProcessInboundBuffers() override;

	/**
	 * Dequeue outbound messages until MaxConsumeTime is reached.
	 * Outbound messages will then be sent via QUIC.
	 */
	virtual void ConsumeOutboundMessages() override;

protected:

	/**
	 * Collects, converts and passes QUIC connection stats to the EndpointManager.
	 */
	virtual void CollectStatistics() override;

public:

	/**
	 * Enqueues an outbound message on the client.
	 *
	 * @param OutboundMessage The outbound message
	 */
	virtual void EnqueueOutboundMessage(const FOutboundMessage& OutboundMessage) override;

	/**
	 * Sends a hello message to the remote endpoint.
	 *
	 * @param RemoteEndpoint The remote endpoint
	 */
	virtual void SendHello(FIPv4Endpoint RemoteEndpoint) override;

	/**
	 * Sends a bye message to the remote endpoint.
	 */
	virtual void SendBye() override;

protected:

	/**
	 * Callback for client connections.
	 *
	 * @param Connection The QUIC connection handle
	 * @param Context The app context (FQuicServer)
	 * @param Event The connection event
	 */
	_Function_class_(QUIC_CONNECTION_CALLBACK)
	static QUIC_STATUS ClientConnectionCallback(HQUIC Connection,
		void* Context, QUIC_CONNECTION_EVENT* Event);

	/**
	 * Callback for client streams.
	 *
	 * @param Stream The QUIC stream handle
	 * @param Context The app context (FQuicServer)
	 * @param Event The stream event
	 */
	_Function_class_(QUIC_STREAM_CALLBACK)
	static QUIC_STATUS ClientStreamCallback(HQUIC Stream,
		void* Context, QUIC_STREAM_EVENT* Event);

protected:

	/**
	 * Handles a connection after the handshake was completed.
	 */
	void OnConnectionConnected();

	/**
	 * Handles a connection after it was killed due to idle timeout.
	 *
	 * @param Status The shutdown event status
	 */
	void OnConnectionShutdownByTransport(QuicUtils::HRESULT Status);

	/**
	 * Handles a connection after it was killed by the peer.
	 *
	 * @param ErrorCode The QUIC error code
	 */
	void OnConnectionShutdownByPeer(QUIC_UINT62 ErrorCode);

	/**
	 * Handles a connection after its shutdown was completed, ready for cleanup.
	 *
	 * @param Connection The connection handle
	 * @param AppCloseInProgress Boolean indicating whether app close is already in progress
	 */
	void OnConnectionShutdownComplete(HQUIC Connection, BOOLEAN AppCloseInProgress);

	/**
	 * Handles a received connection resumption ticket.
	 *
	 * @note Does not do anything at this time
	 */
	void OnConnectionResumptionTicketReceived();

	/**
	 * Handles when a peer started a stream.
	 *
	 * @param Stream The stream handle
	 */
	void OnConnectionPeerStreamStarted(HQUIC Stream);

	/**
	 * Handles a stream when both directions have been shut down, safe for cleanup.
	 *
	 * @param Stream The stream handle
	 */
	void OnStreamShutdownComplete(HQUIC Stream) const;
	
protected:

	/**
	 * Sends the outbound buffers to the remote endpoint.
	 *
	 * @param OutboundMessage The outbound message
	 */
	void SendToStream(const FOutboundMessage& OutboundMessage);

protected:

	/**
	 * Copy data from a stream buffer.
	 *
	 * @param QuicBuffer The inbound QUIC buffer
	 *
	 * @return Whether the operation succeeded
	 */
	bool ReceiveFromStream(const FInboundQuicBuffer& QuicBuffer);

	/**
	 * Prepares inbound data for processing.
	 *
	 * @param QuicBuffer The inbound QUIC buffer
	 */
	void ReachedEndOfStream(const FInboundQuicBuffer& QuicBuffer);

protected:

	/** Holds the QUIC API client connection handle. */
	HQUIC ClientConnection;

	/** Current client connection state. */
	EQuicClientState ConnectionState = EQuicClientState::Connecting;

	/** Indicates whether the transport is closed. */
	bool bTransportClosed = false;

	/** Holds the client config. */
	TSharedRef<FQuicClientConfig> ClientConfig;

};
