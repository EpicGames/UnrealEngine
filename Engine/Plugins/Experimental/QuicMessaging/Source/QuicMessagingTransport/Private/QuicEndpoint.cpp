// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicEndpoint.h"


void FQuicEndpoint::Start()
{
	Thread = FRunnableThread::Create(this, *ThreadName, 128 * 4096, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());

	StartEndpoint();
}


uint32 FQuicEndpoint::Run()
{
	while (!bStopping)
	{
		UpdateEndpoint();
	}

	return 0;
}


void FQuicEndpoint::Stop()
{
	bStopping = true;
}


void FQuicEndpoint::UpdateStatistics()
{
	if (!bUpdatingStatistics)
	{
		bUpdatingStatistics = true;

		const double CurrentTime = FPlatformTime::Seconds();

		if (CurrentTime < (LastStatsPoll + StatsPollInterval.GetTotalSeconds()))
		{
			bUpdatingStatistics = false;
			return;
		}

		CollectStatistics();

		LastStatsPoll = FPlatformTime::Seconds();
		bUpdatingStatistics = false;
	}
}


uint64 FQuicEndpoint::GetStreamId(HQUIC Stream) const
{
	if (MsQuic == nullptr)
	{
		return 0;
	}

	uint64_t StreamId;
	uint32_t StreamIdSize = sizeof(StreamId);

	QUIC_STATUS Status;

	if (QUIC_FAILED(Status = MsQuic->GetParam(Stream,
		QUIC_PARAM_STREAM_ID, &StreamIdSize, &StreamId)))
	{
		return 0;
	}

	return StreamId;
}


/*
 * TODO(vri): Fully implement and fix statistics
 * Tracked in [UCS-5152] Finalize QuicMessaging plugin
 */
QUIC_STATUS FQuicEndpoint::GetConnectionStats(
	HQUIC Connection, QUIC_STATISTICS& ConnectionStats)
{
	uint32_t StatsSize = sizeof(ConnectionStats);

	QUIC_STATUS Status = QUIC_STATUS_SUCCESS;

	/*Status = MsQuic->GetParam(Connection,
		QUIC_PARAM_CONN_STATISTICS, &StatsSize, &ConnectionStats);*/

	return Status;
}


/*
 * TODO(vri): Fully implement and fix statistics
 * Tracked in [UCS-5152] Finalize QuicMessaging plugin
 */
FMessageTransportStatistics FQuicEndpoint::ConvertStatistics(
	FIPv4Endpoint StatsEndpoint, QUIC_STATISTICS& QuicStats)
{
	FMessageTransportStatistics TransportStats;

	TransportStats.TotalBytesSent = QuicStats.Send.TotalBytes;
	TransportStats.PacketsSent = QuicStats.Send.TotalPackets;
	TransportStats.PacketsLost = QuicStats.Send.SuspectedLostPackets - QuicStats.Send.SpuriousLostPackets;
	TransportStats.PacketsAcked = QuicStats.Send.TotalPackets;
	TransportStats.PacketsReceived = QuicStats.Recv.TotalPackets;
	TransportStats.PacketsInFlight = 0;
	TransportStats.WindowSize = 0; // todo
	TransportStats.AverageRTT = { 0 }; // todo
	TransportStats.IPv4AsString = StatsEndpoint.ToString();

	return TransportStats;
}


void FQuicEndpoint::OnStreamSendComplete(const HQUIC Stream, void* Context)
{
	const uint64 StreamId = GetStreamId(Stream);
	const FIPv4Endpoint* PeerAddress = static_cast<FIPv4Endpoint*>(Context);
	const FAddrStream AddressStreamKey(*PeerAddress, StreamId);

	FScopeLock StreamMessagesLock(&StreamMessagesCS);

	FMessageMeta* MessageMeta = StreamMessages.Find(AddressStreamKey);

	if (!MessageMeta)
	{
		return;
	}

	if (!MessageMeta->bIsMetaOnly && !MessageMeta->bMetaSent)
	{
		MessageMeta->bMetaSent = true;
		return;
	}

	StreamMessages.Remove(AddressStreamKey);
}


void FQuicEndpoint::OnStreamReceive(const HQUIC Stream,
	const FIPv4Endpoint PeerAddress, const QUIC_STREAM_EVENT* Event)
{
	const bool bEndOfStream = (Event->RECEIVE.Flags & QUIC_RECEIVE_FLAG_FIN) != 0;

	const uint64 StreamId = GetStreamId(Stream);

	FInboundQuicBuffer InboundBuffer(
		StreamId, Stream, bEndOfStream, PeerAddress, Event->RECEIVE.TotalBufferLength);

	for (uint32 i = 0; i < Event->RECEIVE.BufferCount; ++i)
	{
		const uint32 BufferLength = Event->RECEIVE.Buffers[i].Length;

		if (BufferLength < 1)
		{
			UE_LOGF(LogTemp, Warning,
				"[QuicEndpoint] BufferLength less than 1 on streamId %lld.", StreamId);

			continue;
		}

		InboundBuffer.Data.Add(Event->RECEIVE.Buffers[i].Buffer, BufferLength);
	}

	InboundQuicBuffers.Enqueue(InboundBuffer);
}


void FQuicEndpoint::OnStreamPeerSendAborted(const HQUIC Stream) const
{
	if (MsQuic)
	{
		MsQuic->StreamShutdown(Stream, QUIC_STREAM_SHUTDOWN_FLAG_ABORT, 0);
	}
}


void FQuicEndpoint::RegisterStreamMessage(
	const uint64 StreamId, const FIPv4Endpoint PeerAddress,
	const bool bIsMetaOnly, const FOutboundMessage& OutboundMessage,
	const FQuicBufferRef& MetaBuffer, const FQuicBufferRef& MessageBuffer)
{
	const FMessageMeta MessageMeta(
		bIsMetaOnly, OutboundMessage, MetaBuffer, MessageBuffer);

	const FAddrStream AddressStreamKey(PeerAddress, StreamId);

	FScopeLock StreamMessagesLock(&StreamMessagesCS);

	StreamMessages.Add(AddressStreamKey, MessageMeta);
}


QUIC_ADDR FQuicEndpoint::GetAddressFromEndpoint(FIPv4Endpoint RemoteEndpoint) const
{
	QUIC_ADDR RemoteAddress = {{0}};

#ifdef _WIN32

	inet_pton(AF_INET, TCHAR_TO_ANSI(*RemoteEndpoint.Address.ToString()),
		&RemoteAddress.Ipv4.sin_addr.S_un.S_addr);

#elif defined(__linux__) || defined(__APPLE__)

	inet_pton(AF_INET, TCHAR_TO_ANSI(*RemoteEndpoint.Address.ToString()),
		&RemoteAddress.Ipv4.sin_addr.s_addr);

#endif

	QuicAddrSetFamily(&RemoteAddress, QUIC_ADDRESS_FAMILY_INET);
	QuicAddrSetPort(&RemoteAddress, RemoteEndpoint.Port);

	return RemoteAddress;
}


FIPv4Endpoint FQuicEndpoint::GetEndpointFromAddress(QUIC_ADDR RemoteAddress) const
{
	FIPv4Endpoint RemoteEndpoint;

#ifdef _WIN32

	char Ipv4Buf[INET_ADDRSTRLEN];

	inet_ntop(AF_INET, &RemoteAddress.Ipv4.sin_addr.S_un.S_addr, Ipv4Buf, sizeof(Ipv4Buf));

	RemoteEndpoint.Address.Parse(ANSI_TO_TCHAR(Ipv4Buf), RemoteEndpoint.Address);

#elif defined(__linux__) || defined(__APPLE__)

	char Ipv4Buf[INET_ADDRSTRLEN];

	inet_ntop(AF_INET, &RemoteAddress.Ipv4.sin_addr.s_addr, Ipv4Buf, sizeof(Ipv4Buf));

	RemoteEndpoint.Address.Parse(ANSI_TO_TCHAR(Ipv4Buf), RemoteEndpoint.Address);

#endif

	RemoteEndpoint.Port = QuicAddrGetPort(&RemoteAddress);

	return RemoteEndpoint;
}

