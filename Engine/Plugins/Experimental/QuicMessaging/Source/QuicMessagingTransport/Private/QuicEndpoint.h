// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QuicMessagingTransportPrivate.h"

#include "QuicIncludes.h"

#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "HAL/PlatformTime.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/SingleThreadRunnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "INetworkMessagingExtension.h"

#include "QuicFlags.h"
#include "QuicMessages.h"
#include "QuicBuffers.h"
#include "QuicEndpointConfig.h"
#include "QuicUtils.h"


using FQuicBufferRef = TSharedRef<QUIC_BUFFER, ESPMode::ThreadSafe>;
using FAddrStream = TPair<FIPv4Endpoint, uint64>;


DECLARE_DELEGATE_OneParam(FOnMessageInbound, const FInboundMessage /*InboundMessage*/)

DECLARE_DELEGATE_RetVal_OneParam(FQuicPayloadPtr, FOnSerializeHeader, FMessageHeader& /*MessageHeader*/)

DECLARE_DELEGATE_RetVal_OneParam(FMessageHeader, FOnDeserializeHeader, FQuicPayloadPtr /*HeaderData*/)

DECLARE_DELEGATE_TwoParams(FOnEndpointDisconnected, const FIPv4Endpoint /*DisconnectedEndpoint*/, EQuicEndpointError /*Reason*/)

DECLARE_DELEGATE_OneParam(FOnEndpointConnected, const FIPv4Endpoint& /*ConnectedEndpoint*/)

DECLARE_DELEGATE_OneParam(FOnStatisticsUpdated, const FMessageTransportStatistics /*TransportStats*/)


/**
 * The base class for FQuicServers and FQuicClients.
 */
class FQuicEndpoint
	: public FRunnable
	, private FSingleThreadRunnable
{

protected:

	FQuicEndpoint() = delete;

	/**
	 * Creates and initializes a quic endpoint.
	 * 
	 * @param InEndpointConfig The endpoint config
	 * @param InQuicConfig The QUIC API endpoint config
	 * @param InMsQuic The QUIC API table handle
	 * @param InRegistration The QUIC API registration handle
	 * @param InThreadName The thread name
	 */
	FQuicEndpoint(TSharedRef<FQuicEndpointConfig> InEndpointConfig,
		HQUIC InQuicConfig, const QUIC_API_TABLE* InMsQuic,
		HQUIC InRegistration, const TCHAR* InThreadName)
		: Endpoint(InEndpointConfig->Endpoint)
		, LocalNodeId(InEndpointConfig->LocalNodeId)
		, QuicConfig(InQuicConfig)
		, MsQuic(InMsQuic)
		, Registration(InRegistration)
		, bStopping(false)
		, Thread(nullptr)
		, ThreadName(InThreadName)
		, InboundBuffers(TMap<FAddrStream, FInboundBuffer>())
	{ }

	/**
	 * Destruct QUIC endpoint.
	 */
	virtual ~FQuicEndpoint()
	{
		if (Thread != nullptr)
		{
			Thread->Kill(true);
			delete Thread;
		}
	}

public:

	/**
	 * Start the thread and QUIC endpoint.
	 */
	void Start();

public:

	/**
	 * Continuously updates the endpoint.
	 */
	uint32 Run() override;

	void Stop() override;

protected:

	/**
	 * Starts the configured QUIC endpoint.
	 */
	virtual void StartEndpoint() = 0;

public:

	/**
	 * Stops the QUIC endpoint.
	 */
	virtual void StopEndpoint() = 0;

	/**
	 * Indicates if the endpoint is ready to send and receive messages.
	 */
	virtual bool IsEndpointReady() = 0;

	/**
	 * Indicates whether the endpoint is stopping.
	 */
	bool IsStopping() const
	{
		return bStopping;
	}

	/**
	 * Checks if a QuicServer has a handler with this endpoint.
	 *
	 * @param HandlerEndpoint The handler endpoint
	 * @note This is only valid for QuicServers
	 */
	virtual bool HasHandler(const FIPv4Endpoint HandlerEndpoint) { return false; }

public:

	/**
	 * Periodically called to update the endpoint (consume messages and ping).
	 */
	virtual void UpdateEndpoint() = 0;

protected:

	/**
	 * Periodically called within UpdateEndpoint to updated QUIC connection stats.
	 */
	void UpdateStatistics();

	/**
	 * Collects, converts and passes QUIC connection stats to the EndpointManager.
	 */
	virtual void CollectStatistics() = 0;

public:

	/**
	 * Marks a handler of a QuicServer as authenticated.
	 * 
	 * @param Handler The handler endpoint
	 * @note This is only valid for QuicServers
	 */
	virtual void SetHandlerAuthenticated(const FIPv4Endpoint& Handler) { }

	/**
	 * Disconnects a handler.
	 *
	 * @param HandlerEndpoint The handler endpoint
	 * @note This is only valid for QuicServer
	 */
	virtual void DisconnectHandler(const FIPv4Endpoint& HandlerEndpoint) { }

public:

	/**
	 * Enqueues an outbound message on the endpoint.
	 *
	 * @param OutboundMessage The outbound message
	 */
	virtual void EnqueueOutboundMessage(const FOutboundMessage& OutboundMessage) = 0;

	/**
	 * Sends a hello message to the remote endpoint.
	 *
	 * @param RemoteEndpoint The remote endpoint
	 */
	virtual void SendHello(FIPv4Endpoint RemoteEndpoint) = 0;

	/**
	 * Sends a bye message to the remote endpoint(s).
	 */
	virtual void SendBye() = 0;

protected:

	/**
	 * Helper to get the streamId from a QUIC stream.
	 *
	 * @param Stream The QUIC stream handle
	 */
	uint64 GetStreamId(HQUIC Stream) const;

	/**
	 * Helper to get the connection level stats.
	 *
	 * @param Connection The connection handle
	 * @param ConnectionStats The QUIC statistics container
	 */
	QUIC_STATUS GetConnectionStats(
		HQUIC Connection, QUIC_STATISTICS& ConnectionStats);

	/**
	 * Helper to convert QUIC endpoint statistics to MessageTransportStatistics.
	 *
	 * @param StatsEndpoint The endpoint which the statistics originate from
	 * @param QuicStats The QUIC endpoint statistics
	 */
	FMessageTransportStatistics ConvertStatistics(
		FIPv4Endpoint StatsEndpoint, QUIC_STATISTICS& QuicStats);

protected:

	/**
	 * Struct for QuicCallbacks to pass information to event handlers.
	 */
	struct FStreamCallbackInfo
	{
		/** Holds a pointer to the peer address. */
		FIPv4Endpoint* PeerAddress;

		/** Holds a pointer to either the FQuicServer or FQuicClient. */
		FQuicEndpoint* Endpoint;
	};

protected:

	/**
	 * Handles a stream when all data has been sent and the buffers can be released.
	 *
	 * @param Stream The stream handle
	 * @param Context The app context (QUIC_ADDR)
	 */
	void OnStreamSendComplete(const HQUIC Stream, void* Context);

	/**
	 * Handles received data from the peer.
	 *
	 * @param Stream The stream handle
	 * @param PeerAddress The peer address
	 * @param Event The stream event
	 */
	void OnStreamReceive(const HQUIC Stream,
		const FIPv4Endpoint PeerAddress, const QUIC_STREAM_EVENT* Event);

	/**
	 * Handles a stream when the peer has aborted sending.
	 *
	 * @param Stream The stream handle
	 */
	void OnStreamPeerSendAborted(const HQUIC Stream) const;

protected:

	/**
	 * Register a stream with a MessageId and buffers for cleanup reference.
	 *
	 * @param StreamId The stream ID
	 * @param PeerAddress The peer address
	 * @param bIsMetaOnly Flag indicating whether the message is meta-only
	 * @param OutboundMessage The outbound message
	 * @param MetaBuffer SharedRef to the meta buffer
	 * @param MessageBuffer SharedRef to the message buffer
	 */
	void RegisterStreamMessage(
		const uint64 StreamId, const FIPv4Endpoint PeerAddress,
		const bool bIsMetaOnly, const FOutboundMessage& OutboundMessage,
		const FQuicBufferRef& MetaBuffer, const FQuicBufferRef& MessageBuffer);

protected:

	/**
	 * Convert an endpoint to a QUIC address.
	 *
	 * @param RemoteEndpoint The endpoint to convert
	 */
	QUIC_ADDR GetAddressFromEndpoint(FIPv4Endpoint RemoteEndpoint) const;

	/**
	 * Convert a QUIC address to an endpoint.
	 *
	 * @param RemoteAddress The address to convert
	 */
	FIPv4Endpoint GetEndpointFromAddress(QUIC_ADDR RemoteAddress) const;

public:

	/**
	 * Delegate for when a message has been received by the endpoint and needs to be checked.
	 *
	 * This delegate must be bound before the endpoint thread is started with
	 * the Start() method. It cannot be unbound while the thread is running.
	 * 
	 * @return The delegate
	 */
	FOnMessageInbound& OnMessageInbound()
	{
		return MessageInboundDelegate;
	}

public:

	/**
	 * Delegate for when a header needs to be serialized.
	 *
	 * This delegate must be bound before the endpoint thread is started with
	 * the Start() method. It cannot be unbound while the thread is running.
	 * 
	 * @return The delegate
	 */
	FOnSerializeHeader& OnSerializeHeader()
	{
		return SerializeHeaderDelegate;
	}

	/**
	 * Delegate for when a header needs to be deserialized.
	 *
	 * This delegate must be bound before the endpoint thread is started with
	 * the Start() method. It cannot be unbound while the thread is running.
	 *
	 * @return The delegate
	 */
	FOnDeserializeHeader& OnDeserializeHeader()
	{
		return DeserializeHeaderDelegate;
	}

public:

	/**
	 * Delegate for when an endpoint has disconnected or the connection failed.
	 *
	 * This delegate must be bound before the endpoint thread is started with
	 * the Start() method. It cannot be unbound while the thread is running.
	 *
	 * @return The delegate
	 */
	FOnEndpointDisconnected& OnEndpointDisconnected()
	{
		return EndpointDisconnectedDelegate;
	}

public:

	/**
	 * Delegate for when an endpoint has connected.
	 *
	 * This delegate must be bound before the endpoint thread is started with
	 * the Start() method. It cannot be unbound while the thread is running.
	 *
	 * @return The delegate
	 */
	FOnEndpointConnected& OnEndpointConnected()
	{
		return EndpointConnectedDelegate;
	}

public:

	/**
	 * Delegate for when the statistics for an endpoint are updated.
	 *
	 * This delegate must be bound before the endpoint thread is started with
	 * the Start() method. It cannot be unbound while the thread is running.
	 *
	 * @return The delegate
	 */
	FOnStatisticsUpdated& OnStatisticsUpdated()
	{
		return StatisticsUpdatedDelegate;
	}

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override
	{
		return;
	}

protected:

	/** Holds the IP address (and port) that the socket will be bound to. */
	FIPv4Endpoint Endpoint;

	/** Holds the local node identifier. */
	FGuid LocalNodeId;

	/** Holds the QUIC API config. */
	HQUIC QuicConfig;

	/** Holds the QUIC API handle. */
	const QUIC_API_TABLE* MsQuic;

	/** Holds the QUIC API registration handle. */
	HQUIC Registration;

	/** Flag indicating that thread is stopping. */
	bool bStopping;

	/** The thread object. */
	FRunnableThread* Thread;

	/** The endpoint thread's name. */
	FString ThreadName;

	/** Current endpoint error. */
	EQuicEndpointError EndpointError = EQuicEndpointError::Normal;

protected:

	struct FMessageMeta
	{
		/** Holds a flag indicating whether there is only a meta buffer. */
		bool bIsMetaOnly;

		/** Holds a flag indicating whether the meta buffer has been sent. */
		bool bMetaSent;

		/** Holds the outbound message. */
		FOutboundMessage OutboundMessage;

		/** Holds the meta buffer. */
		FQuicBufferRef MetaBuffer;

		/** Holds the message buffer. */
		FQuicBufferRef MessageBuffer;
		
		/** Creates and initializes a new instance. */
		FMessageMeta(
			const bool bInIsMetaOnly, const FOutboundMessage InOutboundMessage,
			const FQuicBufferRef InMetaBuffer, const FQuicBufferRef InMessageBuffer)
			: bIsMetaOnly(bInIsMetaOnly)
			, bMetaSent(false)
			, OutboundMessage(InOutboundMessage)
			, MetaBuffer(InMetaBuffer)
			, MessageBuffer(InMessageBuffer)
		{ }
	};

	/** Holds QUIC streamIds and associated messages. */
	TMap<FAddrStream, FMessageMeta> StreamMessages;

	/** Holds the StreamMessages critical section. */
	mutable FCriticalSection StreamMessagesCS;

	/** Holds the inbound buffers. */
	TMap<FAddrStream, FInboundBuffer> InboundBuffers;

	/** Holds the InboundBuffers critical section. */
	mutable FCriticalSection InboundBuffersCS;

	/** Holds the inbound MsQuic buffers. */
	TQueue<FInboundQuicBuffer> InboundQuicBuffers;

protected:

	/** Flag indicating if statistics are currently being updated. */
	bool bUpdatingStatistics = false;

	/** Holds timestamp of when statistics have last been updated. */
	double LastStatsPoll = FPlatformTime::Seconds();

	/** Holds the interval for statistics updates. */
	const FTimespan StatsPollInterval = 10000000;

protected:

	/** Holds the message inbound delegate. */
	FOnMessageInbound MessageInboundDelegate;

	/** Holds the serialize header delegate. */
	FOnSerializeHeader SerializeHeaderDelegate;

	/** Holds the deserialize header delegate. */
	FOnDeserializeHeader DeserializeHeaderDelegate;

	/** Holds the endpoint disconnected delegate. */
	FOnEndpointDisconnected EndpointDisconnectedDelegate;

	/** Holds the endpoint connected delegate. */
	FOnEndpointConnected EndpointConnectedDelegate;

	/** Holds the statistics updated delegate. */
	FOnStatisticsUpdated StatisticsUpdatedDelegate;

};
