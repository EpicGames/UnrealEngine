// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicEndpointManager.h"
#include "QuicMessagingTransportPrivate.h"

#include "QuicEndpoint.h"
#include "QuicClient.h"
#include "QuicServer.h"
#include "QuicBuffers.h"
#include "QuicCertificate.h"
#include "QuicIncludes.h"
#include "QuicEndpointConfig.h"

#include "Math/NumericLimits.h"

#include <stdexcept>


/** The protocol name used in the Application Layer Protocol Negotation. */
const QUIC_BUFFER Alpn = { sizeof("unrealquic") - 1, (uint8_t*)"unrealquic" };


FQuicEndpointManager::FQuicEndpointManager(
	TSharedRef<FQuicEndpointConfig> InEndpointConfig)
    : MsQuic(nullptr)
	, Registration(nullptr)
	, CurrentConfiguration(nullptr)
	, LocalNodeId(InEndpointConfig->LocalNodeId)
	, ServerEndpoint(FIPv4Endpoint::Any)
	, EndpointMode(EEndpointMode::Client)
	, EndpointConfig(InEndpointConfig)
	, ReconnectAttempts(TMap<FIPv4Endpoint, uint32>())
	, Endpoints(TMap<FIPv4Endpoint, FQuicEndpointPtr>())
	, KnownNodes(TMap<FGuid, FQuicNodeInfo>())
{
	QUIC_STATUS Status;
    EndpointMode = EEndpointMode::Client;

	if (QUIC_FAILED(Status = MsQuicOpen2(&MsQuic)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not open MsQuic: %ls.",
			*QuicUtils::ConvertResult(Status));

		return;
	}

	constexpr QUIC_REGISTRATION_CONFIG RegistrationConfig
		= { "QuicMessaging", QUIC_EXECUTION_PROFILE_LOW_LATENCY };

	if (QUIC_FAILED(Status = MsQuic->RegistrationOpen(&RegistrationConfig,
		&Registration)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not open registration: %ls.",
			*QuicUtils::ConvertResult(Status));

		return;
	}

	// Setup the discovery timeout tick
	if (EndpointConfig->DiscoveryTimeoutSec > 0)
	{
		DiscoveryTimeoutTickHandle = FTSTicker::GetCoreTicker().AddTicker(
			TEXT("QuicDiscoveryTimeoutTick"), 1, [this](float DeltaSeconds) {

			CheckDiscoveryTimeout();
			return true;

		});
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, 
		"[EndpointManager] Started endpoint manager.");
}


FQuicEndpointManager::~FQuicEndpointManager()
{
	Shutdown();

	if (DiscoveryTimeoutTickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(DiscoveryTimeoutTickHandle);
	}
	
	if (MsQuic)
	{
		if (CurrentConfiguration)
		{
			MsQuic->ConfigurationClose(CurrentConfiguration);
		}

		if (Registration)
		{
			MsQuic->RegistrationShutdown(Registration, QUIC_CONNECTION_SHUTDOWN_FLAG_SILENT, 0);
			MsQuic->RegistrationClose(Registration);
		}

		MsQuicClose(MsQuic);
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, 
		"[EndpointManager] Stopped endpoint manager.");
}


void FQuicEndpointManager::Shutdown()
{
	FScopeLock EndpointsLock(&EndpointsCS);

	if (Endpoints.IsEmpty())
	{
		return;
	}

	for (auto It = Endpoints.CreateIterator(); It; ++It)
	{
		const TSharedPtr<FQuicEndpoint, ESPMode::ThreadSafe>& Endpoint = It->Value;

		if (Endpoint.IsValid())
		{
			if (!Endpoint->IsStopping())
			{
				Endpoint->SendBye();
				Endpoint->StopEndpoint();

				CheckLoseNode(It->Key);
			}
		}

		if (EndpointMode != EEndpointMode::Server)
		{
			It.RemoveCurrent();
		}
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, 
		"[EndpointManager] Shutdown completed.");
}


HQUIC FQuicEndpointManager::LoadClientConfiguration() const
{
	QUIC_SETTINGS QuicSettings = { {0} };

	/** Client idle timeout disabled */
	QuicSettings.IdleTimeoutMs = 0;
	QuicSettings.IsSet.IdleTimeoutMs = 1;

	/** Client connection timeout (5 seconds) */
	QuicSettings.DisconnectTimeoutMs = 5000;
	QuicSettings.IsSet.DisconnectTimeoutMs = 1;

	/** Allow peer to open max unidirectional streams */
	QuicSettings.PeerUnidiStreamCount = std::numeric_limits<uint16_t>::max();
	QuicSettings.IsSet.PeerUnidiStreamCount = 1;

	/** Disable sendbuffering to avoid copying outbound data and delays in sending. */
	QuicSettings.SendBufferingEnabled = 0;
	QuicSettings.IsSet.SendBufferingEnabled = 1;

	/**
	 * Default client config
	 */
	QUIC_CREDENTIAL_CONFIG CredConfig;
	FMemory::Memset(&CredConfig, 0, sizeof(CredConfig));

	CredConfig.Type = QUIC_CREDENTIAL_TYPE_NONE;
	CredConfig.Flags = QUIC_CREDENTIAL_FLAG_CLIENT;

	const TSharedRef<FQuicClientConfig> ClientEndpointConfig
		= StaticCastSharedRef<FQuicClientConfig, FQuicEndpointConfig>(EndpointConfig);

	if (ClientEndpointConfig->ClientVerificationMode == EQuicClientVerificationMode::Pass)
	{
		CredConfig.Flags |= QUIC_CREDENTIAL_FLAG_NO_CERTIFICATE_VALIDATION;
	}

	/**
	 * Initialize client config object
	 */
	HQUIC ClientConfiguration;

	QUIC_STATUS Status = QUIC_STATUS_SUCCESS;

	if (QUIC_FAILED(Status = MsQuic->ConfigurationOpen(Registration, &Alpn, 1,
		&QuicSettings, sizeof(QuicSettings), NULL, &ClientConfiguration)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not open client configuration: %ls.",
			*QuicUtils::ConvertResult(Status));

		return nullptr;
	}

	/**
	 * Load TLS credential part of configuration
	 */
	if (QUIC_FAILED(Status = MsQuic->ConfigurationLoadCredential(ClientConfiguration, &CredConfig)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not load client config credentials: %ls.",
			*QuicUtils::ConvertResult(Status));

		return nullptr;
	}

	return ClientConfiguration;
}


HQUIC FQuicEndpointManager::LoadServerConfiguration(
	FString CertificatePath, FString PrivateKeyPath) const
{
	QUIC_SETTINGS QuicSettings = { {0} };

	/** Server idle timeout disabled */
	QuicSettings.IdleTimeoutMs = 0;
	QuicSettings.IsSet.IdleTimeoutMs = 1;

	/** Server connection timeout (5 seconds) */
	QuicSettings.DisconnectTimeoutMs = 5000;
	QuicSettings.IsSet.DisconnectTimeoutMs = 1;

	/** Server resumption level (allow and 0-RTT) */
	QuicSettings.ServerResumptionLevel = QUIC_SERVER_RESUME_AND_ZERORTT;
	QuicSettings.IsSet.ServerResumptionLevel = 1;

	/** Allow peer to open max unidirectional streams */
	QuicSettings.PeerUnidiStreamCount = std::numeric_limits<uint16_t>::max();
	QuicSettings.IsSet.PeerUnidiStreamCount = 1;

	/** KeepAlive interval (300 milliseconds) */
	QuicSettings.KeepAliveIntervalMs = 300;
	QuicSettings.IsSet.KeepAliveIntervalMs = 1;

	/** Disable sendbuffering to avoid copying outbound data and delays in sending. */
	QuicSettings.SendBufferingEnabled = 0;
	QuicSettings.IsSet.SendBufferingEnabled = 1;

	/**
	 * Credentials
	 */
	QuicCertificateUtils::QUIC_CREDENTIAL_CONFIG_HELPER Config;
	FMemory::Memset(&Config, 0, sizeof(Config));

	Config.CredConfig.Flags = QUIC_CREDENTIAL_FLAG_NONE;

	auto CertFile = StringCast<ANSICHAR>(*CertificatePath);
	auto KeyFile = StringCast<ANSICHAR>(*PrivateKeyPath);

	Config.CertificateFile.CertificateFile = CertFile.Get();
	Config.CertificateFile.PrivateKeyFile = KeyFile.Get();
	Config.CredConfig.Type = QUIC_CREDENTIAL_TYPE_CERTIFICATE_FILE;
	Config.CredConfig.CertificateFile = &Config.CertificateFile;

	/**
	 * Initialize server config object
	 */
	HQUIC ServerConfiguration;

	QUIC_STATUS Status = QUIC_STATUS_SUCCESS;

	if (QUIC_FAILED(Status = MsQuic->ConfigurationOpen(Registration, &Alpn, 1,
		&QuicSettings, sizeof(QuicSettings), NULL, &ServerConfiguration)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not open server configuration: %ls.",
			*QuicUtils::ConvertResult(Status));

		return nullptr;
	}

	if (QUIC_FAILED(Status = MsQuic->ConfigurationLoadCredential(ServerConfiguration, &Config.CredConfig)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not load server config credentials: %ls.",
			*QuicUtils::ConvertResult(Status));

		return nullptr;
	}

	return ServerConfiguration;
}


void FQuicEndpointManager::InitializeServer()
{
	const TSharedRef<FQuicServerConfig> ServerEndpointConfig
		= StaticCastSharedRef<FQuicServerConfig, FQuicEndpointConfig>(EndpointConfig);

	ServerEndpoint = ServerEndpointConfig->Endpoint;

	if (ServerEndpointConfig->AuthenticationMode == EAuthenticationMode::Disabled)
	{
		UE_LOGF(LogQuicMessagingTransport, Warning, 
			"[EndpointManager] Authentication disabled.");
	}
	else
	{
		UE_LOGF(LogQuicMessagingTransport, Verbose, 
			"[EndpointManager] Authentication enabled.");
	}

	if (ServerEndpointConfig->ConnCooldownMode == EConnectionCooldownMode::Enabled)
	{
		UE_LOGF(LogQuicMessagingTransport, Verbose,
			"[EndpointManager] Connection cooldown active: "
			"MaxAttempts: %d, TimePeriod: %ds, Cooldown: %ds, MaxCooldown: %ds.",
			ServerEndpointConfig->ConnCooldownMaxAttempts,
			ServerEndpointConfig->ConnCooldownPeriodSec,
			ServerEndpointConfig->ConnCooldownSec,
			ServerEndpointConfig->ConnCooldownMaxSec);
	}

	{
		FScopeLock EndpointsLock(&EndpointsCS);

		if (!Endpoints.IsEmpty())
		{
			for (auto It = Endpoints.CreateIterator(); It; ++It)
			{
				const TSharedPtr<FQuicEndpoint, ESPMode::ThreadSafe>& Endpoint = It->Value;

				if (Endpoint.IsValid())
				{
					if (!Endpoint->IsStopping())
					{
						Endpoint->StopEndpoint();
					}
				}

				It.RemoveCurrent();
			}
		}
	}

    EndpointMode = EEndpointMode::Server;

	FString PrivateKey = ServerEndpointConfig->PrivateKey;
	FString Certificate = ServerEndpointConfig->Certificate;

	// Generate private key and certificate if none provided
	if (PrivateKey == "" || Certificate == "")
	{
		if (!QuicCertificateUtils::CreateSelfSigned())
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[EndpointManager] Could not generate self-signed certificate.");

			return;
		}

		TTuple<FString, FString> SelfSigned = QuicCertificateUtils::GetSelfSignedPaths();
		Certificate = SelfSigned.Key;
		PrivateKey = SelfSigned.Value;

		UE_LOGF(LogQuicMessagingTransport, Verbose, 
			"[EndpointManager] Generated self-signed certificate.");
	}

	CurrentConfiguration = LoadServerConfiguration(Certificate, PrivateKey);

	if (!CurrentConfiguration)
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Server could not be started.");

		return;
	}

    const TSharedPtr<FQuicEndpoint, ESPMode::ThreadSafe> LocalEndpoint
        = MakeShared<FQuicServer, ESPMode::ThreadSafe>(
		ServerEndpointConfig, CurrentConfiguration,
		MsQuic, Registration, TEXT("QuicServerEndpoint"),Alpn);

    LocalEndpoint->OnMessageInbound().BindRaw(this, &FQuicEndpointManager::ValidateInboundMessage);
    LocalEndpoint->OnSerializeHeader().BindRaw(this, &FQuicEndpointManager::SerializeHeader);
    LocalEndpoint->OnDeserializeHeader().BindRaw(this, &FQuicEndpointManager::DeserializeHeader);
    LocalEndpoint->OnEndpointDisconnected().BindRaw(this, &FQuicEndpointManager::EndpointDisconnected);
	LocalEndpoint->OnEndpointConnected().BindRaw(this, &FQuicEndpointManager::EndpointConnected);
	LocalEndpoint->OnStatisticsUpdated().BindRaw(this, &FQuicEndpointManager::EndpointStatisticsUpdated);

    LocalEndpoint->Start();

	{
		FScopeLock EndpointsLock(&EndpointsCS);
		Endpoints.Add(ServerEndpointConfig->Endpoint, LocalEndpoint);
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[EndpointManager] Configured endpoint manager to run in server mode.");
}


void FQuicEndpointManager::AddClient(const FIPv4Endpoint& ClientEndpoint)
{
    if (EndpointMode == EEndpointMode::Server)
    {
        return;
    }

    {
		FScopeLock EndpointsLock(&EndpointsCS);

		if (const FQuicEndpointPtr* Endpoint = Endpoints.Find(ClientEndpoint))
		{
			UE_LOGF(LogQuicMessagingTransport, Verbose,
				"[EndpointManager] AddClient: endpoint %ls still connected, removing it.",
				*ClientEndpoint.ToString());

			if (!(*Endpoint)->IsStopping())
			{
				(*Endpoint)->StopEndpoint();
			}

			Endpoints.Remove(ClientEndpoint);
		}
    }

	const TSharedRef<FQuicClientConfig> ClientEndpointConfig
		= StaticCastSharedRef<FQuicClientConfig, FQuicEndpointConfig>(EndpointConfig);

	ClientEndpointConfig->Endpoint = ClientEndpoint;

	if (!CurrentConfiguration)
	{
		CurrentConfiguration = LoadClientConfiguration();

		if (!CurrentConfiguration)
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[EndpointManager] Client could not be started.");

			return;
		}
	}

    const TSharedPtr<FQuicEndpoint, ESPMode::ThreadSafe> RemoteEndpoint
        = MakeShared<FQuicClient, ESPMode::ThreadSafe>(
        ClientEndpointConfig, CurrentConfiguration,
		MsQuic, Registration, TEXT("QuicClientEndpoint"));
    
    RemoteEndpoint->OnMessageInbound().BindRaw(this, &FQuicEndpointManager::ValidateInboundMessage);
	RemoteEndpoint->OnSerializeHeader().BindRaw(this, &FQuicEndpointManager::SerializeHeader);
	RemoteEndpoint->OnDeserializeHeader().BindRaw(this, &FQuicEndpointManager::DeserializeHeader);
	RemoteEndpoint->OnEndpointDisconnected().BindRaw(this, &FQuicEndpointManager::EndpointDisconnected);
	RemoteEndpoint->OnStatisticsUpdated().BindRaw(this, &FQuicEndpointManager::EndpointStatisticsUpdated);

	RemoteEndpoint->Start();

    {
		FScopeLock EndpointsLock(&EndpointsCS);
		Endpoints.Add(ClientEndpoint, RemoteEndpoint);
    }

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[EndpointManager] Added client endpoint %ls.",
		*ClientEndpoint.ToString());
}


void FQuicEndpointManager::RemoveClient(const FIPv4Endpoint& ClientEndpoint)
{
	{
		FScopeLock EndpointsLock(&EndpointsCS);

		if (const FQuicEndpointPtr* Endpoint = Endpoints.Find(ClientEndpoint))
		{
			if (!(*Endpoint)->IsStopping())
			{
				(*Endpoint)->StopEndpoint();
			}

			Endpoints.Remove(ClientEndpoint);

			UE_LOGF(LogQuicMessagingTransport, Verbose,
				"[EndpointManager] Removed client endpoint %ls.",
				*ClientEndpoint.ToString());
		}
	}

	CheckLoseNode(ClientEndpoint);
}


/*
 * TODO(vri): Fix reconnect mechanism
 * Tracked in [UCS-5152] Finalize QuicMessaging plugin
 */
void FQuicEndpointManager::ReconnectClient(const FIPv4Endpoint ClientEndpoint)
{
	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[EndpointManager] Reconnecting client %ls.",
		*ClientEndpoint.ToString());

	if (!ReconnectAttempts.Contains(ClientEndpoint))
	{
		ReconnectAttempts.Add(ClientEndpoint, 0);
	}

	if (ReconnectAttempts[ClientEndpoint] == 3)
	{
		return;
	}

	ReconnectAttempts[ClientEndpoint]++;

	AddClient(ClientEndpoint);
}


bool FQuicEndpointManager::IsEndpointAuthenticated(const FGuid& NodeId) const
{
	FScopeLock NodesLock(&NodesCS);

	const FQuicNodeInfo* SearchNode = KnownNodes.Find(NodeId);

	if (!SearchNode)
	{
		return false;
	}

    return SearchNode->bIsAuthenticated;
}


void FQuicEndpointManager::SetEndpointAuthenticated(const FGuid& NodeId)
{
	FScopeLock NodesLock(&NodesCS);

	FQuicNodeInfo* SearchNode = KnownNodes.Find(NodeId);

	if (!SearchNode)
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not find node to set as authenticated: %ls",
			*NodeId.ToString());

		return;
	}

	SearchNode->bIsAuthenticated = true;

    // Mark the handler as authenticated as well
    if (EndpointMode == EEndpointMode::Server)
    {
		FScopeLock EndpointsLock(&EndpointsCS);

		const FQuicEndpointPtr* LocalEndpoint = Endpoints.Find(SearchNode->LocalEndpoint);

		if (!LocalEndpoint)
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[EndpointManager] Could not find server endpoint to set handler as authenticated.");
			
			return;
		}

        (*LocalEndpoint)->SetHandlerAuthenticated(SearchNode->Endpoint);
    }
}


void FQuicEndpointManager::DisconnectNode(const FGuid& NodeId)
{
	FScopeLock NodesLock(&NodesCS);

	const FQuicNodeInfo* SearchNode = KnownNodes.Find(NodeId);

	if (!SearchNode)
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not find node to disconnect: %ls",
			*NodeId.ToString());

		return;
	}

	DisconnectEndpoint(SearchNode->LocalEndpoint);
}


void FQuicEndpointManager::DisconnectEndpoint(const FIPv4Endpoint& Endpoint)
{
	// It's a client, so just remove it
	if (EndpointMode == EEndpointMode::Client)
	{
		RemoveClient(Endpoint);
		return;
	}

	FScopeLock EndpointsLock(&EndpointsCS);

	const FQuicEndpointPtr* LocalEndpoint = Endpoints.Find(Endpoint);

	if (!LocalEndpoint)
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[EndpointManager] Could not find server endpoint to disconnect handler.");

		return;
	}

	(*LocalEndpoint)->DisconnectHandler(Endpoint);
}


void FQuicEndpointManager::EnqueueOutboundMessages(FQuicPayloadPtr Data,
	const TArray<TTuple<FGuid, uint32>>& MessageMetas, const EQuicMessageType MessageType)
{
    if (!Data.IsValid())
    {
        return;
    }

    if (Data->IsEmpty())
    {
        return;
    }

	for (const TTuple<FGuid, uint32>& MessageMeta : MessageMetas)
	{
		const FGuid& Recipient = MessageMeta.Key;

		FIPv4Endpoint LocalAddress;
		FIPv4Endpoint RecipientAddress;

		{
			FScopeLock NodesLock(&NodesCS);

			const FQuicNodeInfo* RecipientNode = KnownNodes.Find(Recipient);

			if (!RecipientNode)
			{
				UE_LOGF(LogQuicMessagingTransport, Error,
					"[EndpointManager] Could not find message recipient node %ls",
					*Recipient.ToString());

				continue;
			}

			LocalAddress = RecipientNode->LocalEndpoint;
			RecipientAddress = RecipientNode->Endpoint;
		}

		FMessageHeader MessageHeader(MessageType, Recipient, LocalNodeId, Data->Num());
		const FQuicPayloadPtr MetaBuffer = SerializeHeaderDelegate.Execute(MessageHeader);
		
		const FOutboundMessage OutboundMessage(
			RecipientAddress, Data, MetaBuffer);

		const FIPv4Endpoint& SendingEndpointAddress
			= (EndpointMode == EEndpointMode::Client) ? RecipientAddress : LocalAddress;

		FScopeLock EndpointsLock(&EndpointsCS);

		const FQuicEndpointPtr* SendingEndpoint = Endpoints.Find(SendingEndpointAddress);

		if (!SendingEndpoint)
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[EndpointManager] Could not find sending endpoint to enqueue message on: %ls",
				*SendingEndpointAddress.ToString());

			continue;
		}

		(*SendingEndpoint)->EnqueueOutboundMessage(OutboundMessage);
	}
}


void FQuicEndpointManager::ValidateInboundMessage(const FInboundMessage InboundMessage)
{
    // Message was generated locally
    if (InboundMessage.MessageHeader.RecipientId == InboundMessage.MessageHeader.SenderId)
    {
        return;
    }

    // Sender guid is not initialized
    if (InboundMessage.MessageHeader.SenderId == FGuid())
    {
        return;
    }

    // Incorrect message segment for server side
    if (EndpointMode == EEndpointMode::Server
        && InboundMessage.MessageHeader.MessageType == EQuicMessageType::AuthenticationResponse)
    {
        return;
    }

    // Incorrect message segment for client side
    if (EndpointMode == EEndpointMode::Client
        && InboundMessage.MessageHeader.MessageType == EQuicMessageType::Authentication)
    {
        return;
    }

    // Node discovery
    if (!KnownNodes.Contains(InboundMessage.MessageHeader.SenderId))
    {
		// Sometimes the first transmitted message (EQuicMessageType::Hello) is broken
		if (!IsMessageValid(InboundMessage))
		{
			SendEndpointHello(InboundMessage);
			return;
		}

        EndpointNodeDiscovered(InboundMessage.MessageHeader.SenderId,
            InboundMessage.Sender, InboundMessage.Receiver);

		SendEndpointHello(InboundMessage);
    }

	if (InboundMessage.MessageHeader.MessageType == EQuicMessageType::Bye)
	{
		EndpointNodeLost(InboundMessage.MessageHeader.SenderId, InboundMessage.Sender);
		EndpointDisconnected(InboundMessage.Sender, EQuicEndpointError::Normal);
	}

	// Don't have to pass these messages on
	if (InboundMessage.MessageHeader.MessageType == EQuicMessageType::Hello
		|| InboundMessage.MessageHeader.MessageType == EQuicMessageType::Bye)
	{
		return;
	}

    MessageValidatedDelegate.ExecuteIfBound(InboundMessage);
}


void FQuicEndpointManager::SendEndpointHello(const FInboundMessage& InboundMessage)
{
	FScopeLock EndpointsLock(&EndpointsCS);

	if (EndpointMode == EEndpointMode::Client)
	{
		if (const FQuicEndpointPtr* Client = Endpoints.Find(InboundMessage.Sender))
		{
			(*Client)->SendHello(InboundMessage.Sender);
		}
	}
}


bool FQuicEndpointManager::IsMessageValid(const FInboundMessage InboundMessage)
{
	if (InboundMessage.MessageHeader.MessageType < EQuicMessageType::Hello
		|| InboundMessage.MessageHeader.MessageType > EQuicMessageType::Bye)
	{
		return false;
	}

	return true;
}


FQuicPayloadPtr FQuicEndpointManager::SerializeHeader(
	FMessageHeader& MessageHeader) const
{
	return SerializeHeaderDelegate.Execute(MessageHeader);
}


FMessageHeader FQuicEndpointManager::DeserializeHeader(
	const FQuicPayloadPtr HeaderData) const
{
	return DeserializeHeaderDelegate.Execute(HeaderData);
}


void FQuicEndpointManager::EndpointNodeDiscovered(const FGuid NodeId,
    const FIPv4Endpoint& NodeEndpoint, const FIPv4Endpoint& LocalEndpoint)
{
	{
		FScopeLock NodesLock(&NodesCS);

		if (FQuicNodeInfo* ExistingNode = KnownNodes.Find(NodeId))
		{
			UE_LOGF(LogQuicMessagingTransport, Display,
				"[EndpointManager] Discovered node already known.");

			if (ExistingNode->Endpoint != NodeEndpoint)
			{
				ExistingNode->Endpoint = NodeEndpoint;
			}

			return;
		}

		const FQuicNodeInfo NodeInfo(NodeEndpoint, NodeId, LocalEndpoint);

		KnownNodes.Add(NodeId, NodeInfo);
	}

    EndpointNodeDiscoveredDelegate.ExecuteIfBound(NodeId);

	if (EndpointMode == EEndpointMode::Client)
	{
		ClientConnectionChangedDelegate.ExecuteIfBound(
			NodeId, NodeEndpoint, EQuicClientConnectionChange::Connected);
	}
}


void FQuicEndpointManager::EndpointNodeLost(const FGuid NodeId, const FIPv4Endpoint LostEndpoint)
{
	{
		FScopeLock NodesLock(&NodesCS);

		const FQuicNodeInfo* ExistingNode = KnownNodes.Find(NodeId);

		if (!ExistingNode)
		{
			UE_LOGF(LogQuicMessagingTransport, Warning,
				"[EndpointManager] Node to forget not found.");

			return;
		}

		if (ExistingNode->Endpoint != LostEndpoint)
		{
			return;
		}

		KnownNodes.Remove(NodeId);
	}

    EndpointNodeLostDelegate.ExecuteIfBound(NodeId);

	if (EndpointMode == EEndpointMode::Client)
	{
		ClientConnectionChangedDelegate.ExecuteIfBound(
			NodeId, LostEndpoint, EQuicClientConnectionChange::Disconnected);
	}
}


TOptional<FGuid> FQuicEndpointManager::GetKnownNodeId(const FIPv4Endpoint& Endpoint) const
{
	FScopeLock NodesLock(&NodesCS);

	for (const auto& NodePair : KnownNodes)
	{
		if (NodePair.Value.Endpoint == Endpoint)
		{
			return TOptional<FGuid>(NodePair.Key);
		}
	}

	return TOptional<FGuid>();
}


bool FQuicEndpointManager::IsEndpointKnown(const FIPv4Endpoint Endpoint) const
{
	FScopeLock NodesLock(&NodesCS);

	bool EndpointKnown = false;

	for (const auto& NodePair : KnownNodes)
	{
		if (NodePair.Value.Endpoint == Endpoint)
		{
			EndpointKnown = true;
			break;
		}
	}

	return EndpointKnown;
}


TArray<FGuid> FQuicEndpointManager::GetKnownNodeIds() const
{
	FScopeLock NodesLock(&NodesCS);

	TArray<FGuid> KnownNodeIds;
	KnownNodes.GetKeys(KnownNodeIds);

	return KnownNodeIds;
}


TArray<FIPv4Endpoint> FQuicEndpointManager::GetKnownEndpoints() const
{
	FScopeLock NodesLock(&NodesCS);

	TArray<FIPv4Endpoint> KnownEndpoints;

	for (const auto& NodePair : KnownNodes)
	{
		KnownEndpoints.Add(NodePair.Value.Endpoint);
	}

	return KnownEndpoints;
}


FMessageTransportStatistics FQuicEndpointManager::GetStats(FGuid NodeId) const
{
	FScopeLock NodesLock(&NodesCS);

	if (FQuicNodeInfo const* NodeInfo = KnownNodes.Find(NodeId))
	{
		return NodeInfo->Statistics;
	}

	return {};
}


void FQuicEndpointManager::EndpointDisconnected(
	const FIPv4Endpoint DisconnectedEndpoint, EQuicEndpointError Reason)
{
	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[EndpointManager] Endpoint disconnected: %ls / Reason: %d.",
		*DisconnectedEndpoint.ToString(), EnumToUnderlyingType(Reason));

	{
		FScopeLock EndpointsLock(&EndpointsCS);

		// Shutdown disconnected client
		if (const FQuicEndpointPtr* Endpoint = Endpoints.Find(DisconnectedEndpoint))
		{
			if (!(*Endpoint)->IsStopping())
			{
				(*Endpoint)->StopEndpoint();
				Endpoints.Remove(DisconnectedEndpoint);
			}
		}
	}

	CheckLoseNode(DisconnectedEndpoint);

	// Attempt reconnect for aborted client side connections
	if (EndpointMode == EEndpointMode::Client && Reason == EQuicEndpointError::ConnectionAbort)
	{
		ReconnectClient(DisconnectedEndpoint);
	}
}


void FQuicEndpointManager::CheckLoseNode(const FIPv4Endpoint LostEndpoint)
{
	FScopeLock NodesLock(&NodesCS);

	// Search for and mark node as lost
	for (const auto& NodePair : KnownNodes)
	{
		if (NodePair.Value.Endpoint == LostEndpoint)
		{
			NodesLock.Unlock();

			EndpointNodeLost(NodePair.Value.NodeId, LostEndpoint);
			break;
		}
	}
}


void FQuicEndpointManager::EndpointConnected(const FIPv4Endpoint& ConnectedEndpoint)
{
	if (EndpointConfig->DiscoveryTimeoutSec == 0)
	{
		return;
	}

	FScopeLock DiscoveryTimeoutsLock(&DiscoveryTimeoutCS);

	FDiscoveryTimeoutEntry TimeoutEntry;

	TimeoutEntry.Endpoint = ConnectedEndpoint;
	TimeoutEntry.Timestamp = FPlatformTime::Seconds() + EndpointConfig->DiscoveryTimeoutSec;

	DiscoveryTimeouts.Add(TimeoutEntry);
}


void FQuicEndpointManager::CheckDiscoveryTimeout()
{
	FScopeLock DiscoveryTimeoutsLock(&DiscoveryTimeoutCS);

	if (DiscoveryTimeouts.IsEmpty())
	{
		return;
	}

	const double Now = FPlatformTime::Seconds();

	for (auto Entry = DiscoveryTimeouts.CreateIterator(); Entry; ++Entry)
	{
		if (Entry->Timestamp < Now)
		{
			return;
		}

		if (!IsEndpointKnown(Entry->Endpoint))
		{
			DisconnectEndpoint(Entry->Endpoint);
		}

		Entry.RemoveCurrent();
	}
}


void FQuicEndpointManager::EndpointStatisticsUpdated(
	const FMessageTransportStatistics TransportStats)
{
	FScopeLock NodesLock(&NodesCS);

	for (auto& NodePair : KnownNodes)
	{
		if (NodePair.Value.Endpoint.ToString() == TransportStats.IPv4AsString)
		{
			NodePair.Value.Statistics = TransportStats;
		}
	}
}

