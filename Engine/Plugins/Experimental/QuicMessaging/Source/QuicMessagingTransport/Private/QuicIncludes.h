// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#endif

// Inspired by
// Engine\Plugins\Media\PixelStreaming\Source\PixelStreaming\Private\WebRTCIncludes.h

// C5105: One of the files included by "msquic.h" has a macro expansion
//		  producing defined, which has undefined behavior
#pragma warning(push)
#pragma warning(disable: 5105)

#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wmissing-braces"
#endif


// This define is needed to provide optional local non-encrypted traffic
#define QUIC_API_ENABLE_INSECURE_FEATURES

#include <msquic.h>
#include <stdio.h>
#include <stdlib.h>


#ifdef __clang__
#pragma clang diagnostic pop
#endif

#pragma warning(pop)


#ifndef UNREFERENCED_PARAMETER
#define UNREFERENCED_PARAMETER(P) (void)(P)
#endif

#define UI UI_ST
#include <openssl/pem.h>
#include <openssl/x509.h>
#undef UI


#if PLATFORM_WINDOWS
THIRD_PARTY_INCLUDES_END
#include "Windows/HideWindowsPlatformTypes.h"
#endif

