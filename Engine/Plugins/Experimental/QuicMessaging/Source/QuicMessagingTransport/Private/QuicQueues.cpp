// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicQueues.h"


void FQuicQueues::ConsumeMessages()
{
	const double CurrentTime = FPlatformTime::Seconds();

	if (CurrentTime < (LastConsumed + ConsumeInterval.GetTotalSeconds()))
	{
		return;
	}

	ConsumeInboundMessages();
	ProcessInboundBuffers();
	ConsumeOutboundMessages();

	LastConsumed = FPlatformTime::Seconds();
}
