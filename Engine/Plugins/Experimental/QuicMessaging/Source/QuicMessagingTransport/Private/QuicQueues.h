// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "Containers/Queue.h"

#include "QuicMessages.h"


class FQuicQueues
{

protected:

	FQuicQueues() = default;

	virtual ~FQuicQueues() = default;

public:

	/**
	 * Periodically called to consume local inbound and outbound messages.
	 * Consumes messages every ConsumeInterval microseconds.
	 */
	void ConsumeMessages();

protected:

    /**
	 * Dequeue inbound messages until MaxConsumeTime is reached.
	 * Inbound messages are passed to QuicEndpointManager for validation.
	 */
	virtual void ConsumeInboundMessages() = 0;

	/**
	 * Dequeue inbound buffers until MaxConsumeTime is reached.
	 * Inbound buffers via QUIC are copied into our buffers until the stream end is indicated.
	 */
	virtual void ProcessInboundBuffers() = 0;

	/**
	 * Dequeue outbound messages until MaxConsumeTime is reached.
	 * Outbound messages will then be sent via QUIC.
	 */
	virtual void ConsumeOutboundMessages() = 0;

protected:

    /** Queue for inbound messages ready to be consumed. */
    TQueue<FInboundMessage> InboundMessages;

	/** Queue for outbound message ready to be sent. */
	TQueue<FOutboundMessage> OutboundMessages;

protected:

    /** Holds current message consume time in microseconds. */
    const FTimespan MaxConsumeTime = 5000;

	/** Holds the interval for message consume. */
	const FTimespan ConsumeInterval = 10000;

    /** Holds timestamp of when messages have last been consumed. */
    double LastConsumed = FPlatformTime::Seconds();

    /** Current id for InboundMessages. */
    uint32 InboundMessagesId = 0;

};
