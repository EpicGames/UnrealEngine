// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicServer.h"

#include "QuicEndpointManager.h"


void FQuicServer::StartEndpoint()
{
	QUIC_STATUS Status;

	const QUIC_ADDR Address = GetAddressFromEndpoint(Endpoint);

	/**
	 * Allocate new listener object
	 */
	if (QUIC_FAILED(Status = MsQuic->ListenerOpen(Registration,
		ServerListenerCallback, this, &ServerListener)))
	{
		const EQuicEndpointError ListenerError = QuicUtils::ConvertQuicStatus(Status);

		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Could not open listener: %ls.",
			*QuicUtils::GetEndpointErrorString(ListenerError));

		return;
	}

	/**
	 * Start listening for incoming connections
	 */
	if (QUIC_FAILED(Status = MsQuic->ListenerStart(ServerListener, &Alpn, 1, &Address)))
	{
		const EQuicEndpointError ListenerError = QuicUtils::ConvertQuicStatus(Status);

		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Could not start listener: %ls.",
			*QuicUtils::GetEndpointErrorString(ListenerError));

		return;
	}

	ConnectionState = EQuicServerState::Running;

	// Setup the connection cooldown check tick
	ConnectionCooldownCheckHandle = FTSTicker::GetCoreTicker().AddTicker(
		TEXT("QuicServerConnCooldownTick"), 1, [this](float DeltaSeconds) {

		CheckConnectionCooldown();
		return true;

	});

	UE_LOGF(LogQuicMessagingTransport, Display,
		"[QuicServer] Started server. Listening on %ls.",
		*GetEndpointFromAddress(Address).ToString());
}


void FQuicServer::StopEndpoint()
{
	if (IsStopping())
	{
		return;
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, 
		"[QuicServer] Stopping server ...");

	Stop();

	if (ConnectionCooldownCheckHandle.IsValid())
	{
		FTSTicker::RemoveTicker(ConnectionCooldownCheckHandle);
	}

	if (ConnectionState < EQuicServerState::Failed)
	{
		ConnectionState = EQuicServerState::Stopped;
	}

	if (MsQuic)
	{
		if (ServerListener)
		{
			MsQuic->ListenerStop(ServerListener);
			MsQuic->ListenerClose(ServerListener);
		}

		ShutdownAllHandlers();
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose, 
		"[QuicServer] Stopped server.");
}


bool FQuicServer::IsEndpointReady()
{
	return ConnectionState == EQuicServerState::Running;
}


void FQuicServer::UpdateEndpoint()
{
	if (IsStopping() || ConnectionState != EQuicServerState::Running)
	{
		return;
	}

	if (Handlers.IsEmpty())
	{
		return;
	}

	ConsumeMessages();
}


void FQuicServer::ConsumeInboundMessages()
{
	if (InboundMessages.IsEmpty())
	{
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	const double ConsumeLimit = CurrentTime + MaxConsumeTime.GetTotalSeconds();

	FInboundMessage InboundMessage;

	while (CurrentTime < ConsumeLimit && InboundMessages.Dequeue(InboundMessage))
	{
		MessageInboundDelegate.ExecuteIfBound(InboundMessage);

		CurrentTime = FPlatformTime::Seconds();
	}
}


void FQuicServer::ProcessInboundBuffers()
{
	if (InboundQuicBuffers.IsEmpty())
	{
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	const double ConsumeLimit = CurrentTime + MaxConsumeTime.GetTotalSeconds();

	FInboundQuicBuffer InboundBuffer;

	while (CurrentTime < ConsumeLimit && InboundQuicBuffers.Dequeue(InboundBuffer))
	{
		if (ReceiveFromStream(InboundBuffer))
		{
			if (InboundBuffer.bEndOfStream)
			{
				ReachedEndOfStream(InboundBuffer);
			}
		}

		CurrentTime = FPlatformTime::Seconds();
	}
}


void FQuicServer::ConsumeOutboundMessages()
{
	if (OutboundMessages.IsEmpty())
	{
		return;
	}

	double CurrentTime = FPlatformTime::Seconds();
	const double ConsumeLimit = CurrentTime + MaxConsumeTime.GetTotalSeconds();
	
	FOutboundMessage OutboundMessage;

	while (CurrentTime < ConsumeLimit && OutboundMessages.Dequeue(OutboundMessage))
	{
		HQUIC HandlerConnection;

		{
			FScopeLock HandlersLock(&HandlersCS);

			const FQuicServerHandlerPtr* ServerHandler
				= Handlers.Find(OutboundMessage.Recipient);

			if (!ServerHandler)
			{
				UE_LOGF(LogQuicMessagingTransport, Error,
					"[QuicServer] Could not find server handler when preparing outbound message: %ls",
					*OutboundMessage.Recipient.ToString());

				continue;
			}

			if ((*ServerHandler)->IsStopping())
			{
				continue;
			}

			if (!(*ServerHandler)->IsRunning())
			{
				OutboundMessages.Enqueue(OutboundMessage);
				continue;
			}

			HandlerConnection = (*ServerHandler)->Connection;
		}

		SendToStream(HandlerConnection, OutboundMessage);

		CurrentTime = FPlatformTime::Seconds();
	}
}


bool FQuicServer::CanHandlerConnect(const FIPv4Endpoint& ConnectingEndpoint) const
{
	const FIPv4Endpoint ClearedEndpoint = ClearEndpointPort(ConnectingEndpoint);

	FScopeLock ConnCooldownLock(&ConnectionCooldownCS);

	return !ConnectionCooldown.Contains(ClearedEndpoint);
}


void FQuicServer::AddConnectionAttempt(const FIPv4Endpoint& ConnectingEndpoint)
{
	if (ServerConfig->ConnCooldownMode != EConnectionCooldownMode::Enabled)
	{
		return;
	}

	const FIPv4Endpoint ClearedEndpoint = ClearEndpointPort(ConnectingEndpoint);

	FScopeLock ConnCooldownLock(&ConnectionCooldownCS);

	const uint64 Now = FMath::Floor(FPlatformTime::Seconds());

	FConnectionAttempts* ExistingAttempts = ConnectionAttempts.Find(ClearedEndpoint);

	if (!ExistingAttempts)
	{
		FConnectionAttempts Attempts;
		Attempts.AttemptTimestamps.Add(Now);

		ConnectionAttempts.Add(ClearedEndpoint, Attempts);

		return;
	}

	ExistingAttempts->AttemptTimestamps.Add(Now);

	const int32 MaximumAttempts = ServerConfig->ConnCooldownMaxAttempts;

	// Prune unnecessary previous attempts
	if (ExistingAttempts->AttemptTimestamps.Num() > MaximumAttempts)
	{
		const int32 RemoveNumAttempts
			= FMath::Abs(MaximumAttempts - ExistingAttempts->AttemptTimestamps.Num());

		ExistingAttempts->AttemptTimestamps.RemoveAt(0, RemoveNumAttempts);
	}

	// Not enough attempts
	if (ExistingAttempts->AttemptTimestamps.Num() < MaximumAttempts)
	{
		return;
	}

	const uint64 EarliestAttemptTimestamp = ExistingAttempts->AttemptTimestamps[0];

	// Earliest attempt is longer ago
	if (Now > (EarliestAttemptTimestamp + ServerConfig->ConnCooldownPeriodSec))
	{
		return;
	}

	// Calculate cooldown multiplier
	const uint64 CooldownMultiplier = FMath::Floor(
		FMath::Exp2(static_cast<float>(ExistingAttempts->NumCooldowns)));

	// Get the cooldown seconds
	uint64 CooldownSeconds = ServerConfig->ConnCooldownSec * CooldownMultiplier;

	// Make sure it doesn't exceed the maximum cooldown
	CooldownSeconds = FMath::Min(CooldownSeconds,
		static_cast<uint64>(ServerConfig->ConnCooldownMaxSec));

	const uint64 CooldownExpires = Now + (CooldownSeconds * CooldownMultiplier);

	ExistingAttempts->NumCooldowns++;
	ExistingAttempts->AttemptTimestamps.Empty();

	ConnectionCooldown.Add(ClearedEndpoint, CooldownExpires);

	UE_LOGF(LogQuicMessagingTransport, Warning,
		"Connection cooldown of %llu seconds for IP %ls.",
		CooldownSeconds, *ClearedEndpoint.ToInternetAddr()->ToString(false));
}


void FQuicServer::CheckConnectionCooldown()
{
	FScopeLock ConnCooldownLock(&ConnectionCooldownCS);

	if (ConnectionCooldown.IsEmpty())
	{
		return;
	}

	if (ServerConfig->ConnCooldownMode == EConnectionCooldownMode::Disabled)
	{
		ConnectionCooldown.Empty();

		return;
	}

	const uint64 Now = FMath::Floor(FPlatformTime::Seconds());

	for (auto CooldownIt = ConnectionCooldown.CreateIterator(); CooldownIt; ++CooldownIt)
	{
		if (Now > CooldownIt->Value)
		{
			CooldownIt.RemoveCurrent();
		}
	}
}


/*
 * TODO(vri): Fully implement and fix server statistics
 * Tracked in [UCS-5152] Finalize QuicMessaging plugin
 */
void FQuicServer::CollectStatistics()
{
	if (IsStopping() || ConnectionState != EQuicServerState::Running)
	{
		return;
	}

	if (MsQuic == nullptr)
	{
		return;
	}

	FScopeLock HandlersLock(&HandlersCS);

	TArray<FIPv4Endpoint> UpdateHandlers;
	Handlers.GetKeys(UpdateHandlers);

	for (FIPv4Endpoint& UpdateEndpoint : UpdateHandlers)
	{
		FQuicServerHandlerPtr Handler = Handlers[UpdateEndpoint];

		if (!Handler.IsValid() || !Handler->IsRunning())
		{
			continue;
		}

		QUIC_STATISTICS Stats{};

		const QUIC_STATUS Status = GetConnectionStats(Handler->Connection, Stats);

		if (Status != QUIC_STATUS_SUCCESS)
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[QuicServer] Could not collect statistics: %ls.",
				*QuicUtils::ConvertResult(Status));

			return;
		}

		if (Handler->NeedRttReset())
		{
			Handler->RttMeasurements.Empty();
			Handler->RttStart = FPlatformTime::Seconds();
		}

		Handler->RttMeasurements.Add(Stats.Rtt);

		FMessageTransportStatistics TransportStats = ConvertStatistics(Handler->Endpoint, Stats);
		TransportStats.AverageRTT = FTimespan::FromMicroseconds(Handler->GetAverageRtt());

		StatisticsUpdatedDelegate.ExecuteIfBound(TransportStats);
	}
}


void FQuicServer::SetHandlerAuthenticated(const FIPv4Endpoint& HandlerEndpoint)
{
	FScopeLock HandlersLock(&HandlersCS);

	if (!Handlers.Contains(HandlerEndpoint))
	{
		return;
	}

	Handlers[HandlerEndpoint]->bAuthenticated = true;
}


void FQuicServer::DisconnectHandler(const FIPv4Endpoint& HandlerEndpoint)
{
	FScopeLock HandlersLock(&HandlersCS);

	const FQuicServerHandlerPtr* Handler = Handlers.Find(HandlerEndpoint);

	if (!Handler)
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Could not find handler to disconnect: %ls.",
			*HandlerEndpoint.ToString());

		return;
	}

	ShutdownHandler((*Handler)->Connection);
}


void FQuicServer::ShutdownAllHandlers() const
{
	FScopeLock HandlersLock(&HandlersCS);

	for (const auto& Handler : Handlers)
	{
		if (Handler.Value->Connection)
		{
			ShutdownHandler(Handler.Value->Connection);
		}
	}
}


void FQuicServer::ShutdownHandler(const HQUIC HandlerConnection) const
{
	if (MsQuic)
	{
		MsQuic->ConnectionShutdown(
			HandlerConnection, QUIC_CONNECTION_SHUTDOWN_FLAG_NONE, 0);
	}
}


void FQuicServer::EnqueueOutboundMessage(const FOutboundMessage& OutboundMessage)
{
	OutboundMessages.Enqueue(OutboundMessage);
}


void FQuicServer::SendHello(FIPv4Endpoint RemoteEndpoint)
{
	if (IsStopping() || ConnectionState != EQuicServerState::Running)
	{
		return;
	}

	FMessageHeader MessageHeader(EQuicMessageType::Hello, FGuid::NewGuid(), LocalNodeId, 0);
	const FQuicPayloadPtr MetaBuffer = SerializeHeaderDelegate.Execute(MessageHeader);

	const FOutboundMessage OutboundMessage(RemoteEndpoint, MetaBuffer);
	
	EnqueueOutboundMessage(OutboundMessage);

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Sent hello to handler %ls.", *RemoteEndpoint.ToString());
}


void FQuicServer::SendBye()
{
	if (IsStopping() || ConnectionState != EQuicServerState::Running)
	{
		return;
	}

	TArray<FIPv4Endpoint> HandlerEndpoints;
	Handlers.GenerateKeyArray(HandlerEndpoints);

	for (FIPv4Endpoint& HandlerEndpoint : HandlerEndpoints)
	{
		SendBye(HandlerEndpoint);
	}
}


void FQuicServer::SendBye(const FIPv4Endpoint& HandlerEndpoint)
{
	if (IsStopping() || ConnectionState != EQuicServerState::Running)
	{
		return;
	}

	FMessageHeader MessageHeader(EQuicMessageType::Bye, FGuid(), LocalNodeId, 0);
	const FQuicPayloadPtr MetaBuffer = SerializeHeaderDelegate.Execute(MessageHeader);

	const FOutboundMessage OutboundMessage(HandlerEndpoint, MetaBuffer);

	HQUIC HandlerConnection;

	{
		FScopeLock HandlersLock(&HandlersCS);

		const FQuicServerHandlerPtr* ServerHandler
			= Handlers.Find(OutboundMessage.Recipient);

		if (!ServerHandler)
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[QuicServer] Could not find server handler when sending bye message: %ls",
				*OutboundMessage.Recipient.ToString());

			return;
		}

		HandlerConnection = (*ServerHandler)->Connection;
	}

	SendToStream(HandlerConnection, OutboundMessage);

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Sent bye to handler %ls.", *HandlerEndpoint.ToString());
}


_Function_class_(QUIC_LISTENER_CALLBACK)
QUIC_STATUS FQuicServer::ServerListenerCallback(HQUIC Listener,
	void* Context, QUIC_LISTENER_EVENT* Event)
{
	UNREFERENCED_PARAMETER(Listener);

	FQuicServer* ServerContext = static_cast<FQuicServer*>(Context);

	QUIC_STATUS Status = QUIC_STATUS_NOT_SUPPORTED;

	/**
	 * Connection attempt by a client
	 */
	if (Event->Type == QUIC_LISTENER_EVENT_NEW_CONNECTION)
	{
		Status = ServerContext->OnListenerNewConnection(Event);
	}

	return Status;
}


_Function_class_(QUIC_CONNECTION_CALLBACK)
QUIC_STATUS FQuicServer::ServerConnectionCallback(HQUIC Connection,
	void* Context, QUIC_CONNECTION_EVENT* Event)
{
	FQuicServer* ServerContext = static_cast<FQuicServer*>(Context);

	switch (Event->Type)
	{

	case QUIC_CONNECTION_EVENT_CONNECTED:
	
		/**
		 * Connection handshake has completed
		 */

		ServerContext->OnConnectionConnected(Connection);

		break;
	

	case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_TRANSPORT:
	
		/**
		 * Idle timeout killed the connection
		 */

		ServerContext->OnConnectionShutdownByTransport(Connection, Event->SHUTDOWN_INITIATED_BY_TRANSPORT.Status);

		break;
	

	case QUIC_CONNECTION_EVENT_SHUTDOWN_INITIATED_BY_PEER:
	
		/**
		 * Peer killed the connection
		 */

		ServerContext->OnConnectionShutdownByPeer(Connection, Event->SHUTDOWN_INITIATED_BY_PEER.ErrorCode);

		break;
	

	case QUIC_CONNECTION_EVENT_SHUTDOWN_COMPLETE:
	
		/**
		 * Connection shutdown complete, ready to cleanup
		 */

		ServerContext->OnConnectionShutdownComplete(Connection, Event->SHUTDOWN_COMPLETE.AppCloseInProgress);

		break;
	

	case QUIC_CONNECTION_EVENT_PEER_STREAM_STARTED:
	{
		/**
		 * Peer started a stream, set local stream callback handler
		 */

		const QUIC_ADDR HandlerAddress = ServerContext->GetRemoteAddress(Connection);
		FIPv4Endpoint* PeerAddress = new FIPv4Endpoint(
			ServerContext->GetEndpointFromAddress(HandlerAddress));

		FStreamCallbackInfo* StreamCallbackInfo = new FStreamCallbackInfo();
		StreamCallbackInfo->Endpoint = ServerContext;
		StreamCallbackInfo->PeerAddress = PeerAddress;

		ServerContext->OnConnectionPeerStreamStarted(Connection, Event->PEER_STREAM_STARTED.Stream);

		ServerContext->MsQuic->SetCallbackHandler(Event->PEER_STREAM_STARTED.Stream,
			reinterpret_cast<void*>(ServerStreamCallback), StreamCallbackInfo);

		break;
	}

	case QUIC_CONNECTION_EVENT_RESUMED:

		/**
		 * TLS resumption of previous session succeeded
		 */

		ServerContext->OnConnectionResumed();

	case QUIC_CONNECTION_EVENT_DATAGRAM_RECEIVED:
	case QUIC_CONNECTION_EVENT_DATAGRAM_SEND_STATE_CHANGED:
	case QUIC_CONNECTION_EVENT_DATAGRAM_STATE_CHANGED:
	case QUIC_CONNECTION_EVENT_IDEAL_PROCESSOR_CHANGED:
	case QUIC_CONNECTION_EVENT_LOCAL_ADDRESS_CHANGED:
	case QUIC_CONNECTION_EVENT_PEER_ADDRESS_CHANGED:
	case QUIC_CONNECTION_EVENT_PEER_CERTIFICATE_RECEIVED:
	case QUIC_CONNECTION_EVENT_PEER_NEEDS_STREAMS:
	case QUIC_CONNECTION_EVENT_RESUMPTION_TICKET_RECEIVED:
	case QUIC_CONNECTION_EVENT_STREAMS_AVAILABLE:

		break;

	default:

		UE_LOGF(LogQuicMessagingTransport, Warning,
			"[QuicServer] Received unhandled connection event: %d", Event->Type);

		break;

	}

	return QUIC_STATUS_SUCCESS;
}


_Function_class_(QUIC_STREAM_CALLBACK)
QUIC_STATUS FQuicServer::ServerStreamCallback(HQUIC Stream,
	void* Context, QUIC_STREAM_EVENT* Event)
{
	const FStreamCallbackInfo* CallbackInfo = static_cast<FStreamCallbackInfo*>(Context);
	FQuicServer* ServerContext = static_cast<FQuicServer*>(CallbackInfo->Endpoint);

	switch (Event->Type)
	{

	case QUIC_STREAM_EVENT_SEND_COMPLETE:

		/**
		 * Previous StreamSend has completed, context returned
		 */

		ServerContext->OnStreamSendComplete(Stream, Event);

		break;

	case QUIC_STREAM_EVENT_RECEIVE:
	
		/**
		 * Data received from the peer
		 */

		ServerContext->OnStreamReceive(Stream, *CallbackInfo->PeerAddress, Event);

		return QUIC_STATUS_PENDING;

	case QUIC_STREAM_EVENT_PEER_SEND_ABORTED:

		/**
		 * Peer aborted its stream send direction
		 */

		ServerContext->OnStreamPeerSendAborted(Stream);

		break;

	case QUIC_STREAM_EVENT_PEER_SEND_SHUTDOWN:

		/**
		 * Peer gracefully shut down its stream send direction
		 */

		break;

	case QUIC_STREAM_EVENT_SHUTDOWN_COMPLETE:
	
		/**
		 * Both stream directions have been shut down, safe to cleanup
		 */

		ServerContext->OnStreamShutdownComplete(Stream);
		delete CallbackInfo;

		break;

	case QUIC_STREAM_EVENT_START_COMPLETE:
	case QUIC_STREAM_EVENT_PEER_ACCEPTED:
	case QUIC_STREAM_EVENT_PEER_RECEIVE_ABORTED:
	case QUIC_STREAM_EVENT_IDEAL_SEND_BUFFER_SIZE:
	case QUIC_STREAM_EVENT_SEND_SHUTDOWN_COMPLETE:

		break;

	default:

		UE_LOGF(LogQuicMessagingTransport, Warning,
			"[QuicServer] Received unhandled stream event: %d", Event->Type);

		break;

	}

	return QUIC_STATUS_SUCCESS;
}


QUIC_STATUS FQuicServer::OnListenerNewConnection(QUIC_LISTENER_EVENT* Event)
{
	const HQUIC HandlerConnection = Event->NEW_CONNECTION.Connection;
	const QUIC_ADDR HandlerAddress = GetRemoteAddress(Event->NEW_CONNECTION.Connection);
	const FIPv4Endpoint HandlerEndpoint = GetEndpointFromAddress(HandlerAddress);

	if (!CanHandlerConnect(HandlerEndpoint))
	{
		return QUIC_STATUS_ABORTED;
	}

	AddConnectionAttempt(HandlerEndpoint);

	if (ServerConfig->EncryptionMode == EEncryptionMode::Disabled)
	{
		constexpr BOOLEAN Value = true;

		MsQuic->SetParam(
			Event->NEW_CONNECTION.Connection,
			QUIC_PARAM_CONN_DISABLE_1RTT_ENCRYPTION,
			sizeof(Value), &Value);
	}

	MsQuic->SetCallbackHandler(Event->NEW_CONNECTION.Connection,
		reinterpret_cast<void*>(ServerConnectionCallback), this);

	const QUIC_STATUS Status = MsQuic->ConnectionSetConfiguration(
		Event->NEW_CONNECTION.Connection, QuicConfig);

	{
		FScopeLock HandlersLock(&HandlersCS);

		const TSharedPtr<FQuicServerHandler, ESPMode::ThreadSafe> Handler
			= MakeShared<FQuicServerHandler, ESPMode::ThreadSafe>(
				HandlerEndpoint, HandlerAddress, HandlerConnection);

		Handlers.Add(HandlerEndpoint, Handler);
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Connection attempt by client: %ls.",
		*HandlerEndpoint.ToString());

	return Status;
}


void FQuicServer::OnConnectionConnected(HQUIC Connection)
{
	const QUIC_ADDR Handler = GetRemoteAddress(Connection);
	const FIPv4Endpoint HandlerEndpoint = GetEndpointFromAddress(Handler);

	{
		FScopeLock HandlersLock(&HandlersCS);

		if (const FQuicServerHandlerPtr* ServerHandler = Handlers.Find(HandlerEndpoint))
		{
			(*ServerHandler)->ConnectionState = EQuicHandlerState::Running;
			(*ServerHandler)->bSentHello = true;
		}
	}

	SendHello(HandlerEndpoint);

	EndpointConnectedDelegate.ExecuteIfBound(HandlerEndpoint);

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Handshake completed. Handler: %ls.", *HandlerEndpoint.ToString());
}


void FQuicServer::OnConnectionShutdownByTransport(HQUIC Connection, QuicUtils::HRESULT Status)
{
	const QUIC_ADDR Handler = GetRemoteAddress(Connection);
	const FIPv4Endpoint HandlerEndpoint = GetEndpointFromAddress(Handler);
	const EQuicEndpointError HandlerError = QuicUtils::ConvertQuicStatus(Status);

	{
		FScopeLock HandlersLock(&HandlersCS);

		if (const FQuicServerHandlerPtr* ServerHandler = Handlers.Find(HandlerEndpoint))
		{
			(*ServerHandler)->ConnectionState = EQuicHandlerState::Disconnected;
			(*ServerHandler)->Error = HandlerError;
		}
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Connection end (by transport). Handler: %ls.",
		*HandlerEndpoint.ToString());

	if (HandlerError != EQuicEndpointError::Normal)
	{
		UE_LOGF(LogQuicMessagingTransport, Verbose,
			"[QuicServer] Handler: %ls / shutdown reason: %ls.",
			*HandlerEndpoint.ToString(), *QuicUtils::GetEndpointErrorString(HandlerError));
	}
}


void FQuicServer::OnConnectionShutdownByPeer(HQUIC Connection, QUIC_UINT62 ErrorCode)
{
	const QUIC_ADDR Handler = GetRemoteAddress(Connection);
	const FIPv4Endpoint HandlerEndpoint = GetEndpointFromAddress(Handler);

	{
		FScopeLock HandlersLock(&HandlersCS);

		if (const FQuicServerHandlerPtr* ServerHandler = Handlers.Find(HandlerEndpoint))
		{
			(*ServerHandler)->ConnectionState = EQuicHandlerState::Disconnected;
		}
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Connection end (by peer). Handler: %ls / Error code: %lld.",
		*HandlerEndpoint.ToString(), ErrorCode);
}


void FQuicServer::OnConnectionShutdownComplete(HQUIC Connection, BOOLEAN AppCloseInProgress)
{
	FScopeLock HandlersLock(&HandlersCS);

	const QUIC_ADDR Handler = GetRemoteAddress(Connection);
	const FIPv4Endpoint HandlerEndpoint = GetEndpointFromAddress(Handler);

	if (!AppCloseInProgress)
	{
		Handlers.Remove(GetEndpointFromAddress(Handler));

		HandlersLock.Unlock();

		EndpointDisconnectedDelegate.ExecuteIfBound(
			HandlerEndpoint, EQuicEndpointError::Normal);

		if (MsQuic)
		{
			MsQuic->ConnectionClose(Connection);
		}
	}

	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Connection shutdown complete. Handler: %ls.",
		*HandlerEndpoint.ToString());
}


void FQuicServer::OnConnectionPeerStreamStarted(HQUIC Connection, HQUIC Stream)
{
	const uint64 StreamId = GetStreamId(Stream);
	const QUIC_ADDR HandlerAddress = GetRemoteAddress(Connection);
	const FIPv4Endpoint HandlerEndpoint = GetEndpointFromAddress(HandlerAddress);

	{
		const FQuicPayloadPtr InboundData = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();

		FScopeLock InboundLock(&InboundBuffersCS);

		const FInboundBuffer InboundBuffer(HandlerAddress, HandlerEndpoint, InboundData);
		const FAddrStream AddressStreamKey(HandlerEndpoint, StreamId);

		InboundBuffers.Add(AddressStreamKey, InboundBuffer);
	}

	UE_LOGF(LogQuicMessagingTransport, VeryVerbose,
		"[QuicServer] Peer stream started. Handler: %ls / StreamId: %lld.",
		*HandlerEndpoint.ToString(), StreamId);
}


void FQuicServer::OnConnectionResumed()
{
	UE_LOGF(LogQuicMessagingTransport, Verbose,
		"[QuicServer] Connection resumed.");
}


void FQuicServer::OnStreamShutdownComplete(HQUIC Stream) const
{
	if (MsQuic)
	{
		MsQuic->StreamClose(Stream);
	}
}


void FQuicServer::SendToStream(
	const HQUIC Connection, const FOutboundMessage OutboundMessage)
{
	HQUIC Stream = nullptr;
	QUIC_STATUS Status;

	FIPv4Endpoint* PeerAddress = new FIPv4Endpoint(OutboundMessage.Recipient);

	FStreamCallbackInfo* StreamCallbackInfo = new FStreamCallbackInfo();
	StreamCallbackInfo->Endpoint = this;
	StreamCallbackInfo->PeerAddress = PeerAddress;

	/**
	 * Create/allocate a new unidirectional stream
	 */
	if (QUIC_FAILED(Status = MsQuic->StreamOpen(Connection, QUIC_STREAM_OPEN_FLAG_UNIDIRECTIONAL,
		ServerStreamCallback, StreamCallbackInfo, &Stream)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Could not open stream: %d.", EnumToUnderlyingType(QuicUtils::ConvertQuicStatus(Status)));
		return;
	}

	/**
	 * Start the stream
	 */
	if (QUIC_FAILED(Status = MsQuic->StreamStart(Stream, QUIC_STREAM_START_FLAG_NONE)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Could not start stream: %d.", EnumToUnderlyingType(QuicUtils::ConvertQuicStatus(Status)));

		MsQuic->StreamClose(Stream);
		return;
	}

	const uint64 StreamId = GetStreamId(Stream);

	/**
	 * Send buffers over the stream
	 */
	const FQuicBufferRef MetaBuffer = MakeShared<QUIC_BUFFER, ESPMode::ThreadSafe>();
	const FQuicBufferRef MessageBuffer = MakeShared<QUIC_BUFFER, ESPMode::ThreadSafe>();

	MetaBuffer->Buffer = OutboundMessage.SerializedHeader->GetData();
	MetaBuffer->Length = OutboundMessage.SerializedHeader->Num();

	RegisterStreamMessage(StreamId, *PeerAddress,
		!OutboundMessage.bHasPayload, OutboundMessage, MetaBuffer, MessageBuffer);

	if (QUIC_FAILED(Status = MsQuic->StreamSend(Stream, &*MetaBuffer, 1,
		OutboundMessage.bHasPayload ? QUIC_SEND_FLAG_NONE : QUIC_SEND_FLAG_FIN, PeerAddress)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Could not send meta on stream: %d.",
			EnumToUnderlyingType(QuicUtils::ConvertQuicStatus(Status)));
		
		MsQuic->StreamShutdown(Stream, QUIC_STREAM_SHUTDOWN_FLAG_ABORT, 0);
		return;
	}

	// This message is header-only
	if (!OutboundMessage.bHasPayload)
	{
		return;
	}

	MessageBuffer->Buffer = OutboundMessage.SerializedPayload->GetData();
	MessageBuffer->Length = OutboundMessage.SerializedPayload->Num();

	if (QUIC_FAILED(Status = MsQuic->StreamSend(Stream,
		&*MessageBuffer, 1, QUIC_SEND_FLAG_FIN, PeerAddress)))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Could not send message on stream: %d.",
			EnumToUnderlyingType(QuicUtils::ConvertQuicStatus(Status)));
		
		MsQuic->StreamShutdown(Stream, QUIC_STREAM_SHUTDOWN_FLAG_ABORT, 0);
	}
}


bool FQuicServer::ReceiveFromStream(const FInboundQuicBuffer& QuicBuffer)
{
	FScopeLock InboundLock(&InboundBuffersCS);

	const FAddrStream AddressStreamKey(
		QuicBuffer.PeerAddress, QuicBuffer.StreamId);

	FInboundBuffer* Buffer = InboundBuffers.Find(AddressStreamKey);

	if (!Buffer)
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Invalid inbound buffer in ReceiveFromStream %lld.",
			QuicBuffer.StreamId);

		return false;
	}

	const FQuicPayloadPtr& InboundData = Buffer->InboundData;

	for (const TPair<uint8*, uint32> InboundBuffer : QuicBuffer.Data)
	{
		Buffer->InboundData->Append(InboundBuffer.Key, InboundBuffer.Value);
	}

	// Retrieve message header
	if (!Buffer->bHeaderDeserialized
		&& InboundData->Num() >= QUIC_MESSAGE_HEADER_SIZE)
	{
		const FQuicPayloadPtr HeaderData = MakeShared<TArray<uint8>, ESPMode::ThreadSafe>();
		HeaderData->Append(InboundData->GetData(), QUIC_MESSAGE_HEADER_SIZE);
		const FMessageHeader MessageHeader = DeserializeHeaderDelegate.Execute(HeaderData);

		Buffer->MessageHeader = MessageHeader;
		Buffer->bHeaderDeserialized = true;

		InboundData->RemoveAt(0, QUIC_MESSAGE_HEADER_SIZE);

		const uint32& SerializedMessageSize = MessageHeader.SerializedMessageSize;
		InboundData->Reserve(SerializedMessageSize);

		// Handler is not authenticated and trying to send message with type Data
		if (AuthenticationMode == EAuthenticationMode::Enabled
			&& MessageHeader.MessageType == EQuicMessageType::Data)
		{
			{
				const FIPv4Endpoint HandlerAddress
					= GetEndpointFromAddress(Buffer->HandlerAddress);

				FScopeLock HandlersLock(&HandlersCS);

				const FQuicServerHandlerPtr* Handler = Handlers.Find(HandlerAddress);

				if (!Handler)
				{
					UE_LOGF(LogQuicMessagingTransport, Error,
						"[QuicServer] Cannot check authentication status of invalid handler %ls.",
						*HandlerAddress.ToString());

					return false;
				}

				if (!(*Handler)->bAuthenticated)
				{
					UE_LOGF(LogQuicMessagingTransport, Error,
						"[QuicServer] Unauthenticated handler sending data.");

					return false;
				}
			}
		}
	}

	// Check authentication message size
	if (Buffer->bHeaderDeserialized)
	{
		if (Buffer->MessageHeader.MessageType == EQuicMessageType::Authentication
			&& InboundData->Num() > static_cast<int32>(ServerConfig->MaxAuthenticationMessageSize))
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[QuicServer] Authentication message size too large.");

			return false;
		}
	}

	InboundLock.Unlock();

	if (MsQuic)
	{
		MsQuic->StreamReceiveComplete(QuicBuffer.Stream, QuicBuffer.BufferLength);
	}

	return true;
}


void FQuicServer::ReachedEndOfStream(const FInboundQuicBuffer& QuicBuffer)
{
	FInboundBuffer InboundBuffer;

	{
		const FAddrStream AddressStreamKey(QuicBuffer.PeerAddress, QuicBuffer.StreamId);

		FScopeLock InboundLock(&InboundBuffersCS);

		if (!InboundBuffers.RemoveAndCopyValue(AddressStreamKey, InboundBuffer))
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[QuicServer] Invalid inbound buffer in ReachedEndOfStream.");

			return;
		}
	}

	const FQuicPayloadPtr& InboundData = InboundBuffer.InboundData;
	const FMessageHeader& MessageHeader = InboundBuffer.MessageHeader;

	// Transmitted message size smaller than expected
	if (InboundData->Num() < static_cast<int32>(MessageHeader.SerializedMessageSize))
	{
		UE_LOGF(LogQuicMessagingTransport, Error,
			"[QuicServer] Transmitted message size smaller than expected %lld / %d / %d.",
			QuicBuffer.StreamId, InboundData->Num(), MessageHeader.SerializedMessageSize);

		return;
	}

	FInboundMessage InboundMessage(InboundData, MessageHeader,
		InboundBuffer.HandlerEndpoint, Endpoint);

	InboundMessages.Enqueue(InboundMessage);
}


QUIC_ADDR FQuicServer::GetRemoteAddress(HQUIC Connection) const
{
	QUIC_ADDR RemoteAddress = {{0}};

	uint32_t RemoteAddressSize = sizeof(RemoteAddress);

	if (MsQuic)
	{
		QUIC_STATUS Status;

		if (QUIC_FAILED(Status = MsQuic->GetParam(Connection,
			QUIC_PARAM_CONN_REMOTE_ADDRESS, &RemoteAddressSize, &RemoteAddress)))
		{
			UE_LOGF(LogQuicMessagingTransport, Error,
				"[QuicServer] Could not get QUIC address from connection: %ls.",
				*QuicUtils::ConvertResult(Status));
		}
	}

	return RemoteAddress;
}


FIPv4Endpoint FQuicServer::ClearEndpointPort(const FIPv4Endpoint& Endpoint)
{
	FIPv4Endpoint ClearedEndpoint = Endpoint;
	ClearedEndpoint.Port = 0;

	return ClearedEndpoint;
}

