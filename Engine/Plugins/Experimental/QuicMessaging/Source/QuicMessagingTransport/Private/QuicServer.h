// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <functional>

#include "Containers/Ticker.h"

#include "QuicServerHandler.h"

#include "QuicEndpoint.h"
#include "QuicQueues.h"
#include "QuicMessages.h"

#include "QuicUtils.h"


// -Woverloaded-virtual:
// Clang thinks the SendHello and SendBye methods are hidden
// overloaded virtual functions because the ones in QuicServer
// require one argument whereas the ones in QuicEndpoint do not
#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Woverloaded-virtual"
#endif


typedef TSharedPtr<FQuicServerHandler, ESPMode::ThreadSafe> FQuicServerHandlerPtr;


/**
 * Implements the QuicServer.
 */
class FQuicServer
	: public FQuicEndpoint
	, public FQuicQueues
{

public:

	FQuicServer() = delete;

	/**
	 * Creates a new QUIC server.
	 */
	FQuicServer(TSharedRef<FQuicServerConfig> InServerConfig,
		HQUIC InQuicConfig, const QUIC_API_TABLE* InMsQuic, HQUIC InRegistration,
		const TCHAR* InThreadName, const QUIC_BUFFER InAlpn)
		: FQuicEndpoint(InServerConfig, InQuicConfig,
			InMsQuic, InRegistration, InThreadName)
		, ServerListener(nullptr)
		, ServerConnection(nullptr)
		, Alpn(InAlpn)
		, AuthenticationMode(InServerConfig->AuthenticationMode)
		, ServerConfig(InServerConfig)
		, Handlers(TMap<FIPv4Endpoint, FQuicServerHandlerPtr>())
		, ConnectionCooldown(TMap<FIPv4Endpoint, uint64>())
		, ConnectionAttempts(TMap<FIPv4Endpoint, FConnectionAttempts>())
	{ }

protected:

	/**
	 * Starts the configured QUIC server.
	 */
	virtual void StartEndpoint() override;

public:

	/**
	 * Stops the QUIC server.
	 */
	virtual void StopEndpoint() override;

	/**
	 * Indicates if the server is ready to send/receive messages.
	 */
	virtual bool IsEndpointReady() override;

public:

	/**
	 * Periodically called to update the endpoint (consume messages).
	 */
	virtual void UpdateEndpoint() override;

protected:

	/**
	 * Dequeue inbound messages until MaxConsumeTime is reached.
	 * Inbound messages are passed to QuicEndpointManager for validation.
	 */
	virtual void ConsumeInboundMessages() override;

	/**
	 * Dequeue inbound buffers until MaxConsumeTime is reached.
	 * Inbound buffers via QUIC are copied into our buffers until the stream end is indicated.
	 */
	virtual void ProcessInboundBuffers() override;

	/**
	 * Dequeue outbound messages until MaxConsumeTime is reached.
	 * Outbound messages will then be sent via QUIC.
	 */
	virtual void ConsumeOutboundMessages() override;

protected:

	/**
	 * Determines if a handler can connect by checking the connection cooldown list.
	 *
	 * @param ConnectingEndpoint The connecting endpoint's address
	 */
	bool CanHandlerConnect(const FIPv4Endpoint& ConnectingEndpoint) const;

	/**
	 * Adds a new connection attempt and triggers connection cooldown if enabled.
	 *
	 * @param ConnectingEndpoint The connecting endpoint's address
	 */
	void AddConnectionAttempt(const FIPv4Endpoint& ConnectingEndpoint);

	/**
	 * Regularly checks and cleans up connection cooldown entries.
	 */
	void CheckConnectionCooldown();

	/**
	 * Collects, converts and passes QUIC connection stats to the EndpointManager.
	 */
	virtual void CollectStatistics() override;

public:

	/**
	 * Marks a handler as authenticated.
	 * 
	 * @param HandlerEndpoint The handler endpoint
	 * @see QuicEndpointManager / Authentication
	 */
	virtual void SetHandlerAuthenticated(const FIPv4Endpoint& HandlerEndpoint) override;

	/**
	 * Disconnects a handler.
	 *
	 * @param HandlerEndpoint The handler endpoint
	 */
	virtual void DisconnectHandler(const FIPv4Endpoint& HandlerEndpoint) override;

protected:

	/**
	 * Shutdown every handler's connection.
	 */
	void ShutdownAllHandlers() const;

	/**
	 * Shutdown a handler's connection.
	 *
	 * @param HandlerConnection The handler connection
	 */
	void ShutdownHandler(const HQUIC HandlerConnection) const;

public:

	/**
	 * Enqueues an outbound message on the server.
	 *
	 * @param OutboundMessage The outbound message
	 */
	virtual void EnqueueOutboundMessage(const FOutboundMessage& OutboundMessage) override;

	/**
	 * Sends a hello message to the remote endpoint.
	 *
	 * @param RemoteEndpoint The remote endpoint
	 */
	virtual void SendHello(FIPv4Endpoint RemoteEndpoint) override;

	/**
	 * Sends a bye message to all handlers.
	 */
	virtual void SendBye() override;

	/**
	 * Sends a bye message to the specified handler.
	 *
	 * @param HandlerEndpoint The handler endpoint
	 */
	void SendBye(const FIPv4Endpoint& HandlerEndpoint);

protected:

	/**
	 * Callback for server listeners.
	 *
	 * @param Listener The QUIC listener handle
	 * @param Context The app context (FQuicServer)
	 * @param Event The listener event
	 */
	_Function_class_(QUIC_LISTENER_CALLBACK)
	static QUIC_STATUS ServerListenerCallback(HQUIC Listener,
		void* Context, QUIC_LISTENER_EVENT* Event);

	/**
	 * Callback for server connections.
	 *
	 * @param Connection The QUIC connection handle
	 * @param Context The app context (FQuicServer)
	 * @param Event The connection event
	 */
	_Function_class_(QUIC_CONNECTION_CALLBACK)
	static QUIC_STATUS ServerConnectionCallback(HQUIC Connection,
		void* Context, QUIC_CONNECTION_EVENT* Event);

	/**
	 * Callback for server streams.
	 *
	 * @param Stream The QUIC stream handle
	 * @param Context The app context (FQuicServer)
	 * @param Event The stream event
	 */
	_Function_class_(QUIC_STREAM_CALLBACK)
	static QUIC_STATUS ServerStreamCallback(HQUIC Stream,
		void* Context, QUIC_STREAM_EVENT* Event);

protected:

	/**
	 * Handles a connection attempt by a client.
	 *
	 * @param Event The listener event
	 */
	QUIC_STATUS OnListenerNewConnection(QUIC_LISTENER_EVENT* Event);

	/**
	 * Handles a connection after the handshake was completed.
	 *
	 * @param Connection The connection handle
	 */
	void OnConnectionConnected(HQUIC Connection);

	/**
	 * Handles a connection after it was killed due to idle timeout.
	 *
	 * @param Connection The connection handle
	 * @param Status The shutdown event status
	 */
	void OnConnectionShutdownByTransport(HQUIC Connection, QuicUtils::HRESULT Status);

	/**
	 * Handles a connection after it was killed by the peer.
	 *
	 * @param Connection The connection handle
	 * @param ErrorCode The QUIC error code
	 */
	void OnConnectionShutdownByPeer(HQUIC Connection, QUIC_UINT62 ErrorCode);

	/**
	 * Handles a connection after its shutdown was completed, ready for cleanup.
	 *
	 * @param Connection The connection handle
	 * @param AppCloseInProgress Boolean indicating whether app close is already in progress
	 */
	void OnConnectionShutdownComplete(HQUIC Connection, BOOLEAN AppCloseInProgress);

	/**
	 * Handles when a peer started a stream.
	 *
	 * @param Connection The connection handle
	 * @param Stream The stream handle
	 */
	void OnConnectionPeerStreamStarted(HQUIC Connection, HQUIC Stream);

	/**
	 * Handles a succeeded TLS resumption of a previous session.
	 */
	void OnConnectionResumed();

	/**
	 * Closes a stream when both directions have been shut down.
	 *
	 * @param Stream The stream handle
	 */
	void OnStreamShutdownComplete(HQUIC Stream) const;

protected:

	/**
	 * Sends the outbound buffers to the remote endpoint.
	 *
	 * @param Connection The QUIC connection handle
	 * @param OutboundMessage The outbound message
	 */
	void SendToStream(
		const HQUIC Connection, const FOutboundMessage OutboundMessage);

protected:

	/**
	 * Copy data from a stream buffer.
	 *
	 * @param QuicBuffer The inbound QUIC buffer
	 *
	 * @return Whether the operation succeeded
	 */
	bool ReceiveFromStream(const FInboundQuicBuffer& QuicBuffer);

	/**
	 * Prepares inbound data for processing.
	 *
	 * @param QuicBuffer The inbound QUIC buffer
	 */
	void ReachedEndOfStream(const FInboundQuicBuffer& QuicBuffer);

protected:

	/**
	 * Get the remote address from a connection.
	 *
	 * @param Connection The QUIC connection handle
	 */
	QUIC_ADDR GetRemoteAddress(HQUIC Connection) const;

	/**
	 * Sets the endpoints port to zero.
	 *
	 * @param Endpoint The endpoint address
	 */
	static FIPv4Endpoint ClearEndpointPort(const FIPv4Endpoint& Endpoint);

protected:

	/** Holds the QUIC API server listener handle. */
	HQUIC ServerListener;

	/** Holds the QUIC API server connection handle. */
	HQUIC ServerConnection;

	/** Current server state. */
	EQuicServerState ConnectionState = EQuicServerState::Starting;

	/** Holds the protocol name for ALPN. */
	QUIC_BUFFER Alpn;

	/** Holds the server authentication mode. */
	EAuthenticationMode AuthenticationMode;

	/** Holds the server config. */
	TSharedRef<FQuicServerConfig> ServerConfig;

protected:

	/** Holds the QuicServerHandlers. */
	TMap<FIPv4Endpoint, FQuicServerHandlerPtr> Handlers;

	/** Holds the handlers critical section. */
	mutable FCriticalSection HandlersCS;

	/** Holds the handle for the connection cooldown check. */
	FTSTicker::FDelegateHandle ConnectionCooldownCheckHandle;

	/** Holds the connection cooldown critical section. */
	mutable FCriticalSection ConnectionCooldownCS;

	/** Holds a map of IPv4 addresses and connection cooldown timestamps. */
	TMap<FIPv4Endpoint, uint64> ConnectionCooldown;

	/** Struct for data concerning connection attempts. */
	struct FConnectionAttempts
	{
		/** Holds the number of previous cooldowns. */
		uint32 NumCooldowns;

		/** Holds the attempt timestamps. */
		TArray<uint64> AttemptTimestamps;

		/** Default constructor. */
		FConnectionAttempts()
			: NumCooldowns(0)
			, AttemptTimestamps(TArray<uint64>())
		{ }
	};

	/** Holds a map of connection attempts. */
	TMap<FIPv4Endpoint, FConnectionAttempts> ConnectionAttempts;

};


#ifdef __clang__
#pragma clang diagnostic pop
#endif

