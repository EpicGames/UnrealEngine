// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "QuicIncludes.h"
#include "QuicFlags.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "HAL/PlatformTime.h"


#define QUIC_STATS_RTTAVG_RESET_AFTER 300000000


struct FQuicServerHandler
{

	/** Holds a flag indicating whether a hello message was sent. */
	bool bSentHello;

	/** Holds a flag indicating whether the handler is authenticated. */
	bool bAuthenticated;

	/** Holds the current handler state. */
	EQuicHandlerState ConnectionState;

	/** Holds the handler endpoint. */
	FIPv4Endpoint Endpoint;

	/** Holds the handler address. */
	QUIC_ADDR Address;

	/** Holds the handler connection. */
	HQUIC Connection;

	/** Holds the handler error. */
	EQuicEndpointError Error;

	/** Holds the latest RTT measurements. */
	TArray<uint32> RttMeasurements;

	/** Holds the time when the RTT measurement sequence started. */
	double RttStart;

	/** Default constructor. */
	FQuicServerHandler()
		: bSentHello(false)
		, bAuthenticated(false)
		, ConnectionState(EQuicHandlerState::Starting)
		, Endpoint(FIPv4Endpoint::Any)
		, Address({})
		, Connection(nullptr)
		, Error(EQuicEndpointError::Normal)
		, RttMeasurements(TArray<uint32>())
		, RttStart(FPlatformTime::Seconds())
	{ }

	/** Creates and initializes a new instance. */
	FQuicServerHandler(const FIPv4Endpoint& InEndpoint,
		QUIC_ADDR InAddress, const HQUIC InConnection)
		: bSentHello(false)
		, bAuthenticated(false)
		, ConnectionState(EQuicHandlerState::Starting)
		, Endpoint(InEndpoint)
		, Address(InAddress)
		, Connection(InConnection)
		, Error(EQuicEndpointError::Normal)
		, RttMeasurements(TArray<uint32>())
		, RttStart(FPlatformTime::Seconds())
	{ }

	bool IsStarting() const
	{
		return ConnectionState == EQuicHandlerState::Starting;
	}

	bool IsRunning() const
	{
		return ConnectionState == EQuicHandlerState::Running;
	}

	bool IsStopping() const
	{
		return ConnectionState == EQuicHandlerState::Stopping;
	}

	bool NeedRttReset() const
	{
		return (FPlatformTime::Seconds() - RttStart) > QUIC_STATS_RTTAVG_RESET_AFTER;
	}

	/**
	 * Gets the average RTT measurement.
	 */
	double GetAverageRtt() const
	{
		uint32 RttSum = 0;

		for (uint32 RttPoint : RttMeasurements)
		{
			RttSum += RttPoint;
		}

		return RttSum / RttMeasurements.Num();
	}

};


