// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuicUtils.h"
#include "QuicIncludes.h"


namespace QuicUtils
{

	FString ConvertResult(HRESULT Result)
	{
		return FString::Printf(TEXT("0x%lx"), Result);
	}


	EQuicEndpointError ConvertQuicStatus(HRESULT Status)
	{
		EQuicEndpointError EndpointError;

		switch (Status)
		{

		case QUIC_STATUS_SUCCESS:

			EndpointError = EQuicEndpointError::Normal;
			break;

		case QUIC_STATUS_OUT_OF_MEMORY:

			EndpointError = EQuicEndpointError::OutOfMemory;
			break;

		case QUIC_STATUS_INVALID_PARAMETER:

			EndpointError = EQuicEndpointError::InvalidParameter;
			break;

		case QUIC_STATUS_INVALID_STATE:

			EndpointError = EQuicEndpointError::InvalidState;
			break;

		case QUIC_STATUS_NOT_SUPPORTED:

			EndpointError = EQuicEndpointError::NotSupported;
			break;

		case QUIC_STATUS_NOT_FOUND:

			EndpointError = EQuicEndpointError::NotFound;
			break;

		case QUIC_STATUS_BUFFER_TOO_SMALL:

			EndpointError = EQuicEndpointError::BufferTooSmall;
			break;

		case QUIC_STATUS_HANDSHAKE_FAILURE:

			EndpointError = EQuicEndpointError::ConnectionHandshake;
			break;

		case QUIC_STATUS_ABORTED:

			EndpointError = EQuicEndpointError::ConnectionAbort;
			break;

		case QUIC_STATUS_ADDRESS_IN_USE:

			EndpointError = EQuicEndpointError::AddressInUse;
			break;

		case QUIC_STATUS_INVALID_ADDRESS:

			EndpointError = EQuicEndpointError::InvalidAddress;
			break;

		case QUIC_STATUS_CONNECTION_TIMEOUT:

			EndpointError = EQuicEndpointError::ConnectionTimeout;
			break;

		case QUIC_STATUS_CONNECTION_IDLE:

			EndpointError = EQuicEndpointError::ConnectionIdle;
			break;

		case QUIC_STATUS_INTERNAL_ERROR:

			EndpointError = EQuicEndpointError::InternalError;
			break;

		case QUIC_STATUS_UNREACHABLE:

			EndpointError = EQuicEndpointError::ServerUnreachable;
			break;

		case QUIC_STATUS_CONNECTION_REFUSED:

			EndpointError = EQuicEndpointError::ConnectionRefused;
			break;

		case QUIC_STATUS_PROTOCOL_ERROR:

			EndpointError = EQuicEndpointError::Protocol;
			break;

		case QUIC_STATUS_VER_NEG_ERROR:

			EndpointError = EQuicEndpointError::VersionNegotiation;
			break;

		case QUIC_STATUS_USER_CANCELED:

			EndpointError = EQuicEndpointError::UserCanceled;
			break;

		case QUIC_STATUS_ALPN_NEG_FAILURE:

			EndpointError = EQuicEndpointError::AlpnNegotiation;
			break;

		case QUIC_STATUS_STREAM_LIMIT_REACHED:

			EndpointError = EQuicEndpointError::StreamLimit;
			break;

		default:

			EndpointError = EQuicEndpointError::Unknown;
			break;

		}

		return EndpointError;
	}


	FString GetEndpointErrorString(EQuicEndpointError Error)
	{
		FString ErrorMessage;

		switch (Error)
		{

		case EQuicEndpointError::Normal:

			ErrorMessage = "";
			break;

		case EQuicEndpointError::OutOfMemory:

			ErrorMessage = "QUIC ran out of memory";
			break;

		case EQuicEndpointError::InvalidParameter:

			ErrorMessage = "Invalid parameter supplied";
			break;

		case EQuicEndpointError::InvalidState:

			ErrorMessage = "Invalid state encountered";
			break;

		case EQuicEndpointError::NotSupported:

			ErrorMessage = "Operation not supported";
			break;

		case EQuicEndpointError::NotFound:

			ErrorMessage = "Object was not found";
			break;

		case EQuicEndpointError::BufferTooSmall:

			ErrorMessage = "Buffer is too small";
			break;

		case EQuicEndpointError::ConnectionHandshake:

			ErrorMessage = "Connection handshake failed";
			break;

		case EQuicEndpointError::ConnectionAbort:

			ErrorMessage = "Connection was aborted by transport";
			break;

		case EQuicEndpointError::AddressInUse:

			ErrorMessage = "Client address/port already in use";
			break;

		case EQuicEndpointError::InvalidAddress:

			ErrorMessage = "Remote address/port invalid";
			break;

		case EQuicEndpointError::ConnectionTimeout:

			ErrorMessage = "Connection timed out while waiting for a response from peer";
			break;

		case EQuicEndpointError::ConnectionIdle:

			ErrorMessage = "Connection timed out from inactivity";
			break;

		case EQuicEndpointError::InternalError:

			ErrorMessage = "Internal Error";
			break;

		case EQuicEndpointError::ServerUnreachable:

			ErrorMessage = "Server is unreachable";
			break;

		case EQuicEndpointError::ConnectionRefused:

			ErrorMessage = "Connection was refused";
			break;

		case EQuicEndpointError::Protocol:

			ErrorMessage = "Encountered protocol error";
			break;

		case EQuicEndpointError::VersionNegotiation:

			ErrorMessage = "Encountered error during version negotiation";
			break;

		case EQuicEndpointError::UserCanceled:

			ErrorMessage = "User canceled handshake";
			break;

		case EQuicEndpointError::AlpnNegotiation:

			ErrorMessage = "Handshake failed to negotiate common ALPN";
			break;

		case EQuicEndpointError::StreamLimit:

			ErrorMessage = "Stream limit was reached";
			break;

		case EQuicEndpointError::Unknown:

			ErrorMessage = "Unknown error occurred";
			break;

		}

		return ErrorMessage;
	}

}
