// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "QuicFlags.h"


namespace QuicUtils
{

	typedef long HRESULT;


	/**
	 * Convert HRESULT to FString.
	 *
	 * @note Most HRESULT values are overriden/extended by msquic
	 * https://github.com/microsoft/msquic/blob/main/docs/TSG.md#understanding-error-codes
	 * https://github.com/microsoft/msquic/blob/main/src/inc/msquic_winuser.h
	 */
	FString ConvertResult(HRESULT Result);

	/**
	 * Convert HRESULT msquic status.
	 */
	EQuicEndpointError ConvertQuicStatus(HRESULT Status);

	FString GetEndpointErrorString(EQuicEndpointError Error);

};




