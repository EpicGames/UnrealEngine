// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

/**
 * Interface for QuicMessagingTransport module
 */
class IQuicMessagingTransportModule : public IModuleInterface
{
public:
	/** Get the QuicMessagingTransport module */
	static IQuicMessagingTransportModule& Get()
	{
		static const FName ModuleName = TEXT("QuicMessagingTransport");
		return FModuleManager::Get().GetModuleChecked<IQuicMessagingTransportModule>(ModuleName);
	}
};
