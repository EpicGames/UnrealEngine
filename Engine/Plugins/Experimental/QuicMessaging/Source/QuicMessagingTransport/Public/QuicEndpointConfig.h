// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "QuicFlags.h"

struct FIPv4Endpoint;
struct FGuid;
enum class EEncryptionMode : uint8;
enum class EAuthenticationMode : uint8;
enum class EConnectionCooldownMode : uint8;
enum class EQuicClientVerificationMode : uint8;
enum class EQuicClientSelfSignedCertificate : uint8;


struct FQuicEndpointConfig
{

	/** Holds the local/remote endpoint. */
	FIPv4Endpoint Endpoint = FIPv4Endpoint::Any;

	/** Holds the local node id. */
	FGuid LocalNodeId = FGuid::NewGuid();

	/** Holds flag indicating whether encryption is enabled. */
	EEncryptionMode EncryptionMode = EEncryptionMode::Enabled;

	/** Holds the discovery timeout in seconds. */
	uint32 DiscoveryTimeoutSec = 0;

};


struct FQuicServerConfig
	: FQuicEndpointConfig
{

	/** Holds the certificate for Quic communication. */
	FString Certificate = "";

	/** Holds the private key. */
	FString PrivateKey = "";

	/** Holds the maximum authentication message size. */
	uint32 MaxAuthenticationMessageSize = 0;

	/** Holds flag indicating whether the server should authenticate clients. */
	EAuthenticationMode AuthenticationMode = EAuthenticationMode::Disabled;

	/** Holds flag indicating whether there is a cooldown for connection attempts. */
	EConnectionCooldownMode ConnCooldownMode = EConnectionCooldownMode::Enabled;

	/** Holds the maximum attempts until the connection cooldown is triggered. */
	uint32 ConnCooldownMaxAttempts = 5;

	/** Holds the time period in seconds within the max attempts must happen. */
	uint32 ConnCooldownPeriodSec = 30;

	/** Holds the connection cooldown in seconds. */
	uint32 ConnCooldownSec = 30;

	/** Holds the maximum connection cooldown in seconds. */
	uint32 ConnCooldownMaxSec = 3600;

};


struct FQuicClientConfig
	: FQuicEndpointConfig
{

	/** Holds flag indicating whether the client should verify the server certificate. */
	EQuicClientVerificationMode ClientVerificationMode = EQuicClientVerificationMode::Verify;

};
