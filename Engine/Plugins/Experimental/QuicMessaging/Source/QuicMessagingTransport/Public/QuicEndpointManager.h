// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "INetworkMessagingExtension.h"

#include "QuicFlags.h"
#include "QuicNodeInfo.h"
#include "QuicMessages.h"

class FQuicEndpoint;
struct FQuicEndpointConfig;
struct QUIC_API_TABLE;
struct QUIC_HANDLE;
struct QUIC_BUFFER;

typedef QUIC_HANDLE* HQUIC;


using FQuicEndpointPtr = TSharedPtr<FQuicEndpoint, ESPMode::ThreadSafe>;


/**
 * QUIC endpoint manager
 *
 * Authentication:
 * - If AuthenticationMode is Enabled, QuicServerHandler will discard
 *   any message with MessageType other than "Authentication" until
 *	 the node associated with the handler is marked as authenticated
 * - Messages with type "Authentication" or "AuthenticationResponse"
 *   can be sent via the extension methods found in
 *	 IQuicNetworkMessagingExtension.h
 * - The MaxAuthenticationMessageSize limit can be set to avoid
 *   reading potentially large malicious authentication messages
 *
 * Client/Server:
 * - The EndpointManager will start in client mode by default
 *   and is converted to a server endpoint when SetServerMode
 *	 is called with additional parameters needed to run a server
 * - QuicServerHandlers will continuously ping the connected
 *   client to keep the connection alive
 */
class QUICMESSAGINGTRANSPORT_API FQuicEndpointManager
{

public:

	FQuicEndpointManager() = delete;

	/**
	 * Initialize endpoint manager with default mode as client.
	 *
	 * @param InEndpointConfig The endpoint config
	 */
	FQuicEndpointManager(
		TSharedRef<FQuicEndpointConfig> InEndpointConfig);

	/**
	 * Default destructor.
	 */
	~FQuicEndpointManager();

public:

	/**
	 * Shutdown endpoint manager and kill client/server threads.
	 */
	void Shutdown();

private:

	/**
	 * Load the client configuration.
	 */
	HQUIC LoadClientConfiguration() const;

	/**
	 * Load the server configuration.
	 *
	 * @param CertificatePath Path to the certificate
	 * @param PrivateKeyPath Path to the private key
	 */
	HQUIC LoadServerConfiguration(
		FString CertificatePath, FString PrivateKeyPath) const;

public:

	/**
	 * Set endpoint manager to operate in server mode and initialize server.

	 * @see Authentication
	 */
	void InitializeServer();

public:

	/**
	 * Add remote endpoint client.
	 *
	 * @param ClientEndpoint The client endpoint that will be connected to
	 */
	void AddClient(const FIPv4Endpoint& ClientEndpoint);

	/**
	 * Remove remote endpoint client.
	 * 
	 * @param ClientEndpoint The client endpoint that will be removed
	 */
	void RemoveClient(const FIPv4Endpoint& ClientEndpoint);

	/**
	 * Reconnect remote endpoint client.
	 *
	 * @param ClientEndpoint The client endpoint that will be reconnected
	 */
	void ReconnectClient(const FIPv4Endpoint ClientEndpoint);

public:

	/**
	 * Check if a QuicServerHandler endpoint is authenticated.
	 * 
	 * @param NodeId The endpoint associated with this node
	 * @return True if the endpoint is authenticated
	 * @note This is only used for QuicServers
	 * @see Authentication
	 */
	bool IsEndpointAuthenticated(const FGuid& NodeId) const;

	/**
	 * Set QuicServerHandler endpoint as authenticated.
	 * 
	 * @param NodeId The endpoint associated with this node
	 * @note This is only used for QuicServers
	 * @see Authentication
	 */
	void SetEndpointAuthenticated(const FGuid& NodeId);

	/**
	 * Disconnect a node.
	 *
	 * @param NodeId The node identifier
	 */
	void DisconnectNode(const FGuid& NodeId);

	/**
	 * Disconnect an endpoint.
	 *
	 * @param Endpoint The endpoint address
	 */
	void DisconnectEndpoint(const FIPv4Endpoint& Endpoint);

public:

	/**
	 * Enqueues outbound messages on all recipient endpoints.
	 * 
	 * @param Data The message data
	 * @param MessageMetas The message recipient node identifiers and associated message ID
	 * @param MessageType The message type
	 * @see Authentication and EQuicMessageType
	 */
	void EnqueueOutboundMessages(FQuicPayloadPtr Data,
		const TArray<TTuple<FGuid, uint32>>& MessageMetas, const EQuicMessageType MessageType);

	/**
	 * Delegate for when a message has been delivered.
	 *
	 * @return The delegate
	 */
	DECLARE_DELEGATE_OneParam(FOnMessageDelivered, const uint32 /*MessageId*/)
	FOnMessageDelivered& OnMessageDelivered()
	{
		return MessageDeliveredDelegate;
	}

protected:

	/**
	 * Validates InboundMessage and passes it to MessageProcessor.
	 * 
	 * @param InboundMessage The inbound message
	 */
	void ValidateInboundMessage(const FInboundMessage InboundMessage);

	/**
	 * Checks if a message is valid.
	 *
	 * @param InboundMessage The inbound message
	 */
	bool IsMessageValid(const FInboundMessage InboundMessage);

	/**
	 * Instructs the appropriate endpoint or handler to send a hello message.
	 *
	 * @param InboundMessage The inbound message
	 */
	void SendEndpointHello(const FInboundMessage& InboundMessage);

public:

	/**
	 * Delegate for when a message has been received and validated.
	 * 
	 * @return The delegate
	 */
	DECLARE_DELEGATE_OneParam(FOnMessageValidated, const FInboundMessage /*InboundMessage*/)
	FOnMessageValidated& OnMessageValidated()
	{
		return MessageValidatedDelegate;
	}

protected:

	/**
	 * Forward for the serialize header delegate.
	 *
	 * @param MessageHeader The message header
	 */
	FQuicPayloadPtr SerializeHeader(
		FMessageHeader& MessageHeader) const;

	/**
	 * Forward for the deserialize header delegate.
	 *
	 * @param HeaderData The header data
	 */
	FMessageHeader DeserializeHeader(
		const FQuicPayloadPtr HeaderData) const;

public:

	/**
	 * Delegate for when a header needs to be serialized.
	 *
	 * @return The delegate
	 */
	DECLARE_DELEGATE_RetVal_OneParam(FQuicPayloadPtr, FOnSerializeHeader, FMessageHeader& /*MessageHeader*/)
	FOnSerializeHeader& OnSerializeHeader()
	{
		return SerializeHeaderDelegate;
	}

	/**
	 * Delegate for when a header needs to be deserialized.
	 *
	 * @return The delegate
	 */
	DECLARE_DELEGATE_RetVal_OneParam(FMessageHeader, FOnDeserializeHeader, FQuicPayloadPtr /*HeaderData*/)
	FOnDeserializeHeader& OnDeserializeHeader()
	{
		return DeserializeHeaderDelegate;
	}

public:

	/**
	 * Register discovered endpoint node.
	 * 
	 * @param NodeId The node ID
	 * @param NodeEndpoint The node endpoint address
	 * @param LocalEndpoint The local endpoint
	 */
	void EndpointNodeDiscovered(const FGuid NodeId,
		const FIPv4Endpoint& NodeEndpoint, const FIPv4Endpoint& LocalEndpoint);

	/**
	 * De-register lost endpoint node.
	 * 
	 * @param NodeId The node ID
	 * @param LostEndpoint The lost endpoint address
	 */
	void EndpointNodeLost(const FGuid NodeId, const FIPv4Endpoint LostEndpoint);

	/**
	 * Delegate for when a node was discovered.
	 * 
	 * @return The delegate
	 */
	DECLARE_DELEGATE_OneParam(FOnEndpointNodeDiscovered, const FGuid& /*NodeId*/)
	FOnEndpointNodeDiscovered& OnEndpointNodeDiscovered()
	{
		return EndpointNodeDiscoveredDelegate;
	}
	
	/**
	 * Delegate for when a node was lost.
	 * 
	 * @return The delegate
	 */
	DECLARE_DELEGATE_OneParam(FOnEndpointNodeLost, const FGuid& /*NodeId*/)
	FOnEndpointNodeLost& OnEndpointNodeLost()
	{
		return EndpointNodeLostDelegate;
	}

	/**
	 * Delegate for when a client connection changes.
	 *
	 * @return The delegate
	 */
	DECLARE_DELEGATE_ThreeParams(FOnClientConnectionChanged,
		const FGuid& /*NodeId*/, const FIPv4Endpoint& /*RemoteEndpoint*/,
		const EQuicClientConnectionChange /*ConnectionState*/)
	FOnClientConnectionChanged& OnClientConnectionChanged()
	{
		return ClientConnectionChangedDelegate;
	}

	/**
	 * Get the NodeId of a known endpoint.
	 *
	 * @param Endpoint The endpoint
	 *
	 * @return Optional FGuid if the node was found
	 */
	TOptional<FGuid> GetKnownNodeId(const FIPv4Endpoint& Endpoint) const;

	/**
	 * Get a list of known NodeIds.
	 */
	TArray<FGuid> GetKnownNodeIds() const;

	/**
	 * Check if an endpoint is associated with a known node.
	 *
	 * @param Endpoint The endpoint
	 */
	bool IsEndpointKnown(const FIPv4Endpoint Endpoint) const;

	/**
	 * Get a list of the known endpoints.
	 */
	TArray<FIPv4Endpoint> GetKnownEndpoints() const;

	/**
	 * Get the network/transport statistics for a node.
	 *
	 * @param NodeId The node ID
	 */
	FMessageTransportStatistics GetStats(FGuid NodeId) const;

protected:

	/**
	 * Removes a disconnected endpoint.
	 *
	 * @param DisconnectedEndpoint The disconnected endpoint
	 * @param Reason The disconnect reason
	 */
	void EndpointDisconnected(
		const FIPv4Endpoint DisconnectedEndpoint, EQuicEndpointError Reason);

	/**
	 * Checks if the lost endpoint node is still registered and removes it.
	 *
	 * @param LostEndpoint The lost endpoint
	 */
	void CheckLoseNode(const FIPv4Endpoint LostEndpoint);

	/**
	 * Adds a DiscoveryTimeout entry if discovery timeouts are enabled.
	 *
	 * @param ConnectedEndpoint The connected endpoint
	 */
	void EndpointConnected(const FIPv4Endpoint& ConnectedEndpoint);


	/**
	 * Regularly called to check if any of the endpoints have reached
	 * the discovery timeout. Timed out endpoints will be disconnected.
	 */
	void CheckDiscoveryTimeout();

	/**
	 * Updates a KnownNode with new endpoint transport statistics.
	 *
	 * @param TransportStats The endpoint transport statistics
	 */
	void EndpointStatisticsUpdated(
		const FMessageTransportStatistics TransportStats);

private:

	/** Holds the QUIC API table handle. */
	const QUIC_API_TABLE* MsQuic;

	/** Holds the QUIC API registration handle. */
	HQUIC Registration;

	/** Holds the current QUIC API configuration handle. */
	HQUIC CurrentConfiguration;

private:

	/** Holds the local node identifier. */
	FGuid LocalNodeId;

	/** Holds the local server endpoint that the socket will be bound to. */
	FIPv4Endpoint ServerEndpoint;

	/** Flag indicating the endpoint mode. */
	EEndpointMode EndpointMode;

	/** Holds the endpoint config. */
	TSharedRef<FQuicEndpointConfig> EndpointConfig;

	/** Holds the collection of reconnect attempts. */
	TMap<FIPv4Endpoint, uint32> ReconnectAttempts;

	/** Holds the collection of active QUIC endpoints. */
	TMap<FIPv4Endpoint, FQuicEndpointPtr> Endpoints;

	/** Mutex protecting access to the endpoints map. */
	mutable FCriticalSection EndpointsCS;

	/** Holds the collection of known remote nodes. */
	TMap<FGuid, FQuicNodeInfo> KnownNodes;

	/** Mutex protecting access to the nodes map. */
	mutable FCriticalSection NodesCS;

private:

	/** Holds the handle for the discovery timeout tick. */
	FTSTicker::FDelegateHandle DiscoveryTimeoutTickHandle;

	struct FDiscoveryTimeoutEntry
	{

		/** Holds the address of the connected endpoint. */
		FIPv4Endpoint Endpoint = FIPv4Endpoint::Any;

		/** Holds the connection timestamp. */
		double Timestamp = 0;

	};

	/** Holds the connection timestamps for discovery timeout purposes. */
	TArray<FDiscoveryTimeoutEntry> DiscoveryTimeouts;

	/** Mutex protecting the discovery timeouts array. */
	mutable FCriticalSection DiscoveryTimeoutCS;

private:

	/** Holds a delegate to be invoked when a message was delivered to all endpoints. */
	FOnMessageDelivered MessageDeliveredDelegate;

	/**
	 * Holds a delegate to be invoked when a message has been received and validated,
	 * ready to be serialized by the local endpoint.
	 */
	FOnMessageValidated MessageValidatedDelegate;

private:

	/** Holds the serialize header delegate. */
	FOnSerializeHeader SerializeHeaderDelegate;

	/** Holds the deserialize header delegate. */
	FOnDeserializeHeader DeserializeHeaderDelegate;

private:

	/** Holds a delegate to be invoked when a node was discovered. */
	FOnEndpointNodeDiscovered EndpointNodeDiscoveredDelegate;

	/** Holds a delegate to be invoked when a node was lost. */
	FOnEndpointNodeLost EndpointNodeLostDelegate;

	/** Holds a delegate to be invoked when a client connection has changed. */
	FOnClientConnectionChanged ClientConnectionChangedDelegate;

};
