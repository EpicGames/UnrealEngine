// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "CoreTypes.h"


/**
 * Enumerates endpoint mode.
 */
enum class EEndpointMode : uint8
{
	/** EndpointManager acts as a client. */
	Client,

	/** EndpointManager acts as a server. */
	Server
};


/**
 * Enumerates authentication mode.
 */
enum class EAuthenticationMode : uint8
{
	/** Client nodes have to be authenticated before sending MessageType. */
	Enabled,

	/** Authentication is disabled, all nodes can connect and send data. */
	Disabled
};


/**
 * Enumerates encryption mode.
 *
 * When disabled, encryption will only be used for the handshake.
 * Further communication will be unencrypted.
 *
 * @note Both endpoints must have the same setting to successfully connect.
 */
enum class EEncryptionMode : uint8
{
	/** Encryption is enabled. */
	Enabled,

	/** Encryption is disabled. */
	Disabled
};


/**
 * Enumerates connection cooldown mode.
 *
 * When enabled, multiple successive attempts to connect from
 * the same IP address will trigger a cooldown.
 *
 * @note This is a server-only flag
 */
enum class EConnectionCooldownMode : uint8
{
	/** Connection cooldown is enabled. */
	Enabled,

	/** Connection cooldown is disabled. */
	Disabled
};


/**
 * Enumerates client connection state.
 */
enum class EQuicClientState : uint8
{
	/** Client is setting up connection. */
	Connecting,

	/** Client is connected and running. */
	Connected,

	/** Client could not connect. */
	Failed,

	/** Client is disconnected. */
	Disconnected,

	/** Client connection is closed. */
	Closed,

	/** Client is being destroyed. */
	Stopped
};


/**
 * Enumerates client connection changes.
 */
enum class EQuicClientConnectionChange : uint8
{
	/** Client has connected. */
	Connected,

	/** Client has disconnected. */
	Disconnected
};


/**
 * Enumerates client certificate verification.
 */
enum class EQuicClientVerificationMode : uint8
{
	/** Verify server certificate. */
	Verify,

	/** Pass server certificate verification. */
	Pass
};


/**
 * Enumerates client self signed certificates mode.
 */
enum class EQuicClientSelfSignedCertificate : uint8
{
	/** Reject self-signed certificates. */
	Reject,

	/** Allow self-signed certificates. */
	Allow
};


/**
 * Enumerates server state.
 */
enum class EQuicServerState : uint8
{
	/** Server is starting. */
	Starting,

	/** Server is running. */
	Running,

	/** Server encountered error and can't recover. */
	Failed,

	/** Server is stopped. */
	Stopped
};


/**
 * Enumerates server handler state.
 */
enum class EQuicHandlerState : uint8
{
	/** Handler is starting. */
	Starting,

	/** Handler is running. */
	Running,

	/** Handler is stopping. */
	Stopping,

	/** Handler connection failed. */
	Failed,

	/** Handler has disconnected. */
	Disconnected
};


/**
 * Enumerates message type.
 *
 * @see QuicEndpointManager - Authentication
 */
enum class EQuicMessageType : uint8
{
	/** Hello, header-only. */
	Hello,

	/** Authentication (from client to server) */
	Authentication,

	/** Authentication response (from server to client) */
	AuthenticationResponse,

	/** Data */
	Data,

	/** Bye, header-only. */
	Bye

};


/**
 * Enumerates endpoint error.
 */
enum class EQuicEndpointError : uint16
{
	/** No endpoint error. */
	Normal,

	/** QUIC ran out of memory. */
	OutOfMemory,

	/** Invalid parameter supplied. */
	InvalidParameter,

	/** Invalid state encountered. */
	InvalidState,

	/** Operation not supported. */
	NotSupported,

	/** Object was not found. */
	NotFound,

	/** Buffer is too small. */
	BufferTooSmall,

	/** Connection handshake failed. */
	ConnectionHandshake,

	/** Connection was aborted by transport. */
	ConnectionAbort,

	/** Client address/port already in use. */
	AddressInUse,

	/** Remote address/port invalid. */
	InvalidAddress,

	/** Connection timed out while waiting for a response from peer. */
	ConnectionTimeout,

	/** Connection timed out from inactivity. */
	ConnectionIdle,

	/** Internal error. */
	InternalError,

	/** Server is unreachable. */
	ServerUnreachable,

	/** Connection was refused. */
	ConnectionRefused,

	/** Encountered protocol error. */
	Protocol,

	/** Encountered error during version negotiation. */
	VersionNegotiation,

	/** User canceled handshake. */
	UserCanceled,

	/** Handshake failed to negotiate common ALPN. */
	AlpnNegotiation,

	/** Stream limit was reached. */
	StreamLimit,

	/** Unknown error. */
	Unknown

};


