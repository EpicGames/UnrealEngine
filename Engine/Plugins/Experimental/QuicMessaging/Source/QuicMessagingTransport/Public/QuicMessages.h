// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <memory>

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "GenericPlatform/GenericPlatform.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/Guid.h"

#include "QuicFlags.h"


#define QUIC_MESSAGE_HEADER_SIZE 96

using FQuicPayload = TArray<uint8>;
using FQuicPayloadPtr = TSharedPtr<FQuicPayload, ESPMode::ThreadSafe>;


/**
 * Structure for QUIC message headers.
 */
struct FMessageHeader
{
    /** Holds the message type. */
    EQuicMessageType MessageType;

    /** Holds the recipient node id. */
    FGuid RecipientId;

    /** Holds the sender node id. */
    FGuid SenderId;

    /** Holds the serialized message size. */
    uint32 SerializedMessageSize;

    /** Default constructor. */
    FMessageHeader()
	    : MessageType(EQuicMessageType::Hello)
		, RecipientId(FGuid::NewGuid())
		, SenderId(FGuid::NewGuid())
		, SerializedMessageSize(0)
	{ }

    /** Creates and initializes a new instance. */
    FMessageHeader(const EQuicMessageType InMessageType, const FGuid InRecipientId,
        const FGuid InSenderId, const uint32 InSerializedMessageSize)
        : MessageType(InMessageType)
        , RecipientId(InRecipientId)
        , SenderId(InSenderId)
        , SerializedMessageSize(InSerializedMessageSize)
    { }
};


/**
 * Structure for unserialized inbound messages.
 */
struct FInboundMessage
{
    /** Holds the message data. */
	FQuicPayloadPtr UnserializedMessage;

    /** Holds the message header. */
    FMessageHeader MessageHeader;

    /** Holds the sender's network endpoint. */
    FIPv4Endpoint Sender;

    /** Holds the receiver's network endpoint. */
    FIPv4Endpoint Receiver;

    /** Default constructor. */
    FInboundMessage()
	    : UnserializedMessage(nullptr)
		, MessageHeader(FMessageHeader())
		, Sender(FIPv4Endpoint::Any)
		, Receiver(FIPv4Endpoint::Any)
	{ }

    /** Creates and initializes a new instance. */
    FInboundMessage(
		FQuicPayloadPtr InUnserializedMessage,
		const FMessageHeader& InMessageHeader,
		const FIPv4Endpoint& InSender, const FIPv4Endpoint& InReceiver)
        : UnserializedMessage(InUnserializedMessage)
        , MessageHeader(InMessageHeader)
        , Sender(InSender)
		, Receiver(InReceiver)
    { }
};


/**
 * Structure for serialized outbound messages.
 */
struct FOutboundMessage
{
	/** Holds the recipient endpoint address. */
	FIPv4Endpoint Recipient;

	/** Flag indicating whether this message has a payload. */
	bool bHasPayload;

    /** Holds the serialized payload data. */
	FQuicPayloadPtr SerializedPayload;

	/** Holds the serialized message header. */
	FQuicPayloadPtr SerializedHeader;

	/** Default constructor. */
	FOutboundMessage()
		: Recipient(FIPv4Endpoint::Any)
		, bHasPayload(false)
		, SerializedPayload(nullptr)
		, SerializedHeader(nullptr)
	{ }

    /** Creates and initializes a new instance. */
    FOutboundMessage(const FIPv4Endpoint InRecipient,
		const FQuicPayloadPtr InSerializedMessage,
		const FQuicPayloadPtr InSerializedHeader)
	    : Recipient(InRecipient)
		, bHasPayload(true)
		, SerializedPayload(InSerializedMessage)
		, SerializedHeader(InSerializedHeader)
    { }

	/** Creates and initializes a new header-only instance. */
	FOutboundMessage(const FIPv4Endpoint InRecipient,
		const FQuicPayloadPtr InSerializedHeader)
		: Recipient(InRecipient)
		, bHasPayload(false)
		, SerializedPayload(nullptr)
		, SerializedHeader(InSerializedHeader)
	{}
};
