// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/Guid.h"
#include "INetworkMessagingExtension.h"


/**
 * Structure for known remote endpoints.
 */
struct FQuicNodeInfo
{
	/** Holds the node's endpoint. */
	FIPv4Endpoint Endpoint;

	/** Holds the endpoint's node identifier. */
	FGuid NodeId;

	/** Holds the node's local quic endpoint. */
	FIPv4Endpoint LocalEndpoint;

	/** Denotes wheter this remote node is authenticated or not. */
	bool bIsAuthenticated;

	/** Various transport statistics for this endpoint */
	FMessageTransportStatistics Statistics;

	/** Default constructor. */
	FQuicNodeInfo()
		: Endpoint(FIPv4Endpoint::Any)
		, NodeId(FGuid::NewGuid())
		, LocalEndpoint(FIPv4Endpoint::Any)
		, bIsAuthenticated(false)
		, Statistics({})
	{ }

	/** Creates and initializes a new instance. */
	FQuicNodeInfo(const FIPv4Endpoint& InEndpoint, const FGuid& InNodeId,
		const FIPv4Endpoint& InLocalEndpoint)
		: Endpoint(InEndpoint)
		, NodeId(InNodeId)
		, LocalEndpoint(InLocalEndpoint)
		, bIsAuthenticated(false)
		, Statistics({})
	{ }
};
