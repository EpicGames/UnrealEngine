// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class QuicMessagingTransport : ModuleRules
	{
		public QuicMessagingTransport(ReadOnlyTargetRules Target) : base(Target)
		{
			bEnableExceptions = true;
			//bUseRTTI = true;

			if (Target.Platform == UnrealTargetPlatform.Win64) {

				PublicSystemLibraries.AddRange(
					new string[] {
						"shlwapi.lib",
						"crypt32.lib"
					}
				);

			}
			
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"MsQuic",
					"MsQuicRuntime"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Messaging",
					"Networking",
					"SSL"
				}
			);

			PrivateIncludePaths.AddRange(
				new string[] {
					"QuicMessagingTransport/Private",
					"QuicMessagingTransport/Public",
				}
			);

			AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenSSL");
		}
	}
}
